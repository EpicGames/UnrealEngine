// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Virtualization/VirtualizationSystem.h"

/** 
 * The functions in this header are not set in stone and are likely to be changed without warning.
 * Once they are finalized they can be moved to the public utils header for general use.
 */
namespace UE::Virtualization::Utils
{

/**
 * Adds filter reasons for a package that would only be calculated during the virtualization process and not stored in the payload trailer
 * Currently this is used to fix up reporting in our commandlets and console commands so that it is easier for users to see why payloads
 * have not been virtualized. This is only required because some filters are applied at package save time (when we know the owning asset
 * type) and others at submit time.
 * The two stage filtering does need fixing properly but I am not sure which way we want to do it, this is why we have an experimental
 * utility to fix the reporting as I do not want to commit to an API change while still unsure on the direction we should take.
 * 
 * NOTE: Only works if the "Default" virtualization system is used.
 * 
 * @param PackagePath			Used to check for EPayloadFilterReason::Path
 * @param SizeOnDisk			Used to check for EPayloadFilterReason::MinSize
 * @param CurrentFilterFlags	The filters for the payload stored in the payload trailer.
 * @return						The final filter reason used by the payload during the virtualization process.
 */
VIRTUALIZATION_API EPayloadFilterReason FixFilterFlags(FStringView PackagePath, uint64 SizeOnDisk, EPayloadFilterReason CurrentFilterFlags);

} //namespace UE::Virtualization::Utils