// Copyright Epic Games, Inc. All Rights Reserved.

#include "PackageRehydrationProcess.h"

#include "HAL/FileManager.h"
#include "Internationalization/Internationalization.h"
#include "Misc/PackageName.h"
#include "Misc/PackagePath.h"
#include "Misc/ScopeExit.h"
#include "Misc/ScopedSlowTask.h"
#include "PackageUtils.h"
#include "Serialization/MemoryArchive.h"
#include "UObject/Linker.h"
#include "UObject/Package.h"
#include "UObject/PackageTrailer.h"
#include "UObject/UObjectGlobals.h"
#include "Virtualization/VirtualizationSystem.h"
#include "VirtualizationSourceControlUtilities.h"

#define LOCTEXT_NAMESPACE "Virtualization"

namespace UE::Virtualization
{

/** 
 * Archive allowing serialization to a fixed sized memory buffer. Exceeding the
 * length of the buffer will cause the archive to log a warning and set its
 * error state to true.
 */
class FFixedBufferWriterArchive : public FMemoryArchive
{
public:
	FFixedBufferWriterArchive() = delete;

	/** 
	 * @param InBuffer Pointer to the buffer to write to
	 * @param InLength Total length (in bytes) that can be written to the buffer
	 */
	FFixedBufferWriterArchive(uint8* InBuffer, int64 InLength)
		: Buffer(InBuffer)
		, Length(InLength)
	{
		SetIsSaving(true);
	}

	virtual ~FFixedBufferWriterArchive() = default;
	
	/** Returns how much space remains in the internal buffer. */
	int64 GetRemainingSpace() const
	{
		return Length - Offset;
	}

private:

	virtual void Serialize(void* Data, int64 Num) override
	{
		if (Offset + Num <= Length)
		{
			FMemory::Memcpy(&Buffer[Offset], Data, Num);
			Offset += Num;
		}
		else
		{
			UE_LOGF(LogSerialization, Error, "Attempting to write %lld bytes to a FFixedBufferWriterArchive with only %lld bytes of space remaining", Num, GetRemainingSpace());
			SetError();
		}
	}

	uint8* Buffer = nullptr;
	int64 Length = INDEX_NONE;
};

/** Utility for opening a package for reading, including localized error handling */
TUniquePtr<FArchive> OpenFileForReading(const FString& FilePath, TArray<FText>& OutErrors)
{
	TUniquePtr<FArchive> FileHandle(IFileManager::Get().CreateFileReader(*FilePath));
	if (!FileHandle)
	{
		FText Message = FText::Format(LOCTEXT("VAHydration_OpenFailed", "Unable to open the file '{0}' for reading"),
			FText::FromString(FilePath));

		OutErrors.Add(MoveTemp(Message));
	}

	return FileHandle;
}

/** Utility for serializing data from a package, including localized error handling */
bool ReadData(TUniquePtr<FArchive>& Ar, void* DstBuffer, int64 BytesToRead, const FString& FilePath, TArray<FText>& OutErrors)
{
	check(Ar.IsValid());

	Ar->Serialize(DstBuffer, BytesToRead);

	if (!Ar->IsError())
	{
		return true;
	}
	else
	{
		FText Message = FText::Format(LOCTEXT("VAHydration_ReadFailed", "Failed to read {0} bytes from file {1}"),
			BytesToRead,
			FText::FromString(FilePath));

		OutErrors.Add(MoveTemp(Message));
		return false;
	}
}

/** Shared rehydration code */
bool TryRehydrateBuilder(const FString& FilePath, FPackageTrailer& OutTrailer, FPackageTrailerBuilder& OutBuilder, TArray<FText>& OutErrors)
{
	if (!FPackageName::IsPackageFilename(FilePath))
	{
		return false; // Only rehydrate valid packages
	}

	if (!FPackageTrailer::TryLoadFromFile(FilePath, OutTrailer))
	{
		return false; // Only rehydrate packages with package trailers
	}

	TArray<FIoHash> VirtualizedPayloads = OutTrailer.GetPayloads(EPayloadStorageType::Virtualized);
	if (VirtualizedPayloads.IsEmpty())
	{
		return false; // If the package has no virtualized payloads then we can skip the rest
	}

	{
		TUniquePtr<FArchive> FileAr = OpenFileForReading(FilePath, OutErrors);
		if (!FileAr)
		{
			return false;
		}

		OutBuilder = FPackageTrailerBuilder::CreateFromTrailer(OutTrailer, *FileAr, FilePath);
	}

	int32 PayloadsHydrated = 0;

	IVirtualizationSystem& System = IVirtualizationSystem::Get();

	TArray<FPullRequest> Requests;
	Requests.Reserve(VirtualizedPayloads.Num());

	for (const FIoHash& Id : VirtualizedPayloads)
	{
		Requests.Emplace(FPullRequest(Id));
	}

	if (!System.PullData(Requests))
	{
		FText Message = FText::Format(LOCTEXT("VAHydration_PullFailed", "Unable to pull the data for the package '{0}'"),
			FText::FromString(FilePath));
		OutErrors.Add(Message);

		return false;
	}

	for (const FPullRequest& Request : Requests)
	{
		if (OutBuilder.UpdatePayloadAsLocal(Request.GetIdentifier(), Request.GetPayload()))
		{
			PayloadsHydrated++;
		}
		else
		{
			FText Message = FText::Format(LOCTEXT("VAHydration_UpdateStatusFailed", "Unable to update the status for the payload '{0}' in the package '{1}'"),
				FText::FromString(LexToString(Request.GetIdentifier())),
				FText::FromString(FilePath));
			OutErrors.Add(Message);

			return false;
		}
	}

	check(OutBuilder.GetNumVirtualizedPayloads() == 0);

	return PayloadsHydrated > 0;
}

void RehydratePackages(TConstArrayView<FString> PackagePaths, ERehydrationOptions Options, FRehydrationResult& OutResult)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::Virtualization::RehydratePackagesOnDisk);

	IVirtualizationSystem& System = IVirtualizationSystem::Get();

	if (!System.IsEnabled())
	{
		return;
	}

	const double StartTime = FPlatformTime::Seconds();

	ON_SCOPE_EXIT
	{
		OutResult.TimeTaken = FPlatformTime::Seconds() - StartTime;
		UE_LOGF(LogVirtualization, Display, "Rehydration process complete");
		UE_LOGF(LogVirtualization, Verbose, "Rehydration process took %.3f(s)", OutResult.TimeTaken);
	};

	FScopedSlowTask Progress(1.0f, LOCTEXT("VAHydration_TaskOnDisk", "Re-hydrating Assets On Disk..."));
	Progress.MakeDialog();

	TArray<TPair<FString, FString>> PackagesToReplace;

	// TODO: Should we consider a way to batch this so many payloads can be downloaded at once?
	//       Running this over a large project would be much faster if we could batch. An easy way
	//       to do this might be to gather all of the payloads needed and do a prefetch first?

	double Time = FPlatformTime::Seconds();

	TSet<FString> ConsideredPackages;
	ConsideredPackages.Reserve(PackagePaths.Num());

	// Attempt to rehydrate the packages
	for(int32 Index = 0; Index < PackagePaths.Num(); ++Index)
	{
		const FString& FilePath = PackagePaths[Index];

		bool bIsDuplicate = false;
		ConsideredPackages.Add(FilePath, &bIsDuplicate);
		if (bIsDuplicate)
		{
			UE_LOGF(LogVirtualization, Verbose, "Skipping duplicate package entry '%ls'", *FilePath);
			continue; // Skip duplicate packages
		}

		FPackageTrailer Trailer;
		FPackageTrailerBuilder Builder;

		if (TryRehydrateBuilder(FilePath, Trailer, Builder, OutResult.Errors))
		{
			FString NewPackagePath = DuplicatePackageWithNewTrailer(FilePath, Trailer, Builder, OutResult.Errors);

			if (!NewPackagePath.IsEmpty())
			{
				PackagesToReplace.Emplace(FilePath, MoveTemp(NewPackagePath));
			}
			else
			{
				// Error?
				return;
			}
		}

		if (FPlatformTime::Seconds() - Time > 10.0)
		{
			const float ProgressPercent = ((float)Index / (float)PackagePaths.Num()) * 100.0f;
			UE_LOGF(LogVirtualization, Display, "%d/%d - %.1f%%", Index, PackagePaths.Num(), ProgressPercent);

			Time = FPlatformTime::Seconds();
		}
	}

	const int32 NumSkippedPackages = PackagePaths.Num() - ConsideredPackages.Num();
	ConsideredPackages.Empty();

	UE_CLOGF(NumSkippedPackages > 0, LogVirtualization, Warning, "Discarded %d duplicate package paths", NumSkippedPackages);

	// We need to reset the loader of any loaded package that should have its package file replaced
	for (const TPair<FString, FString>& Pair : PackagesToReplace)
	{
		const FString& OriginalFilePath = Pair.Key;

		FString PackageName;
		if (FPackageName::TryConvertFilenameToLongPackageName(OriginalFilePath, PackageName))
		{
			UPackage* Package = FindObjectFast<UPackage>(nullptr, *PackageName);
			if (Package != nullptr)
			{
				UE_LOGF(LogVirtualization, Verbose, "Detaching '%ls' from disk so that it can be rehydrated", *OriginalFilePath);
				ResetLoadersForSave(Package, *OriginalFilePath);
			}
		}
	}

	// Should we try to check out packages from revision control?
	if (EnumHasAnyFlags(Options, ERehydrationOptions::Checkout))
	{
		TArray<FString> FilesToCheckState;
		FilesToCheckState.Reserve(PackagesToReplace.Num());

		for (const TPair<FString, FString>& Pair : PackagesToReplace)
		{
			FilesToCheckState.Add(Pair.Key);
		}

		if (!TryCheckoutFiles(FilesToCheckState, OutResult.Errors, &OutResult.CheckedOutPackages))
		{
			return;
		}
	}

	for (const TPair<FString, FString>& Pair : PackagesToReplace)
	{
		const FString& OriginalFilePath = Pair.Key;
		const FString& NewPackagePath = Pair.Value;

		if (CanWriteToFile(OriginalFilePath))
		{
			const int64 OriginalSize = IFileManager::Get().GetStatData(*OriginalFilePath).FileSize;
			const int64 HydratedSize = IFileManager::Get().GetStatData(*NewPackagePath).FileSize;

			if (IFileManager::Get().Move(*OriginalFilePath, *NewPackagePath))
			{
				OutResult.RehydratedPackages.Add(OriginalFilePath);

				OutResult.PreOperationSize += OriginalSize;
				OutResult.PostOperationSize += HydratedSize;

				// Could consider path name here to match the virtualization process.
				UE_LOGF(LogVirtualization, Verbose, "Hydrating %ls: %ls -> %ls",
					*OriginalFilePath,
					*FText::AsMemory(OriginalSize).ToString(),
					*FText::AsMemory(HydratedSize).ToString());
			}
			else
			{
				FText Message = FText::Format(LOCTEXT("VAHydration_MoveFailed", "Unable to replace the package '{0}' with the hydrated version"),
					FText::FromString(OriginalFilePath));

				OutResult.AddError(MoveTemp(Message));
				return;
			}
		}
		else
		{
			FText Message = FText::Format(
				LOCTEXT("VAHydration_PackageLocked", "The package file '{0}' has virtualized payloads but is locked for modification and cannot be hydrated"),
				FText::FromString(OriginalFilePath));

			OutResult.AddError(MoveTemp(Message));
			return;
		}
	}

	if (OutResult.PreOperationSize != OutResult.PostOperationSize)
	{
		UE_LOGF(LogVirtualization, Display, "Total hydration increase %ls (%ls -> %ls)",
			*FText::AsMemory(OutResult.PostOperationSize - OutResult.PreOperationSize).ToString(),
			*FText::AsMemory(OutResult.PreOperationSize).ToString(),
			*FText::AsMemory(OutResult.PostOperationSize).ToString());
	}

	UE_LOGF(LogVirtualization, Display, "Hydrated %d package(s)", OutResult.RehydratedPackages.Num());
}

void RehydratePackages(TConstArrayView<FString> PackagePaths, uint64 PaddingAlignment, TArray<FText>& OutErrors, TArray<FSharedBuffer>& OutPackages, TArray<FRehydrationInfo>* OutInfo)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::Virtualization::RehydratePackagesInMem);

	IVirtualizationSystem& System = IVirtualizationSystem::Get();

	FScopedSlowTask Progress(1.0f, LOCTEXT("VAHydration_TaskInMem", "Re-hydrating Assets In Memory..."));
	Progress.MakeDialog();

	OutPackages.Empty(PackagePaths.Num());
	if (OutInfo != nullptr)
	{
		OutInfo->Empty(PackagePaths.Num());
	}

	for (const FString& FilePath : PackagePaths)
	{
		FUniqueBuffer FileBuffer;

		FPackageTrailer Trailer;
		FPackageTrailerBuilder Builder;

		if (System.IsEnabled() && TryRehydrateBuilder(FilePath, Trailer, Builder, OutErrors))
		{
			TUniquePtr<FArchive> FileAr = OpenFileForReading(FilePath, OutErrors);
			if (!FileAr)
			{
				return;
			}

			const int64 OriginalFileLength = FileAr->TotalSize();
			const int64 TruncatedFileLength = OriginalFileLength - Trailer.GetTrailerLength();
			const int64 RehydratedTrailerLength = (int64)Builder.CalculateTrailerLength();
			const int64 RehydratedFileLength = TruncatedFileLength + RehydratedTrailerLength;
			const int64 BufferLength = Align(RehydratedFileLength, PaddingAlignment);

			FileBuffer = FUniqueBuffer::Alloc((uint64)BufferLength);

			if (!ReadData(FileAr, FileBuffer.GetData(), TruncatedFileLength, FilePath, OutErrors))
			{
				return;
			}

			FFixedBufferWriterArchive Ar((uint8*)FileBuffer.GetData() + TruncatedFileLength, RehydratedTrailerLength);
			if (!Builder.BuildAndAppendTrailer(nullptr, Ar))
			{
				FText Message = FText::Format(LOCTEXT("VAHydration_TrailerFailed", "Failed to create a new trailer for file {0}"),
					FText::FromString(FilePath));

				OutErrors.Add(MoveTemp(Message));

				return;
			}

			check(Ar.GetRemainingSpace() == 0);	

			if (OutInfo != nullptr)
			{
				FRehydrationInfo& Info = OutInfo->AddDefaulted_GetRef();
				Info.OriginalSize = OriginalFileLength;
				Info.RehydratedSize = RehydratedFileLength;
				Info.NumPayloadsRehydrated = Builder.GetNumLocalPayloads() - Trailer.GetNumPayloads(EPayloadStorageType::Local);
			}
		}
		else
		{
			TUniquePtr<FArchive> FileAr = OpenFileForReading(FilePath, OutErrors);
			if (!FileAr)
			{
				return;
			}

			const int64 FileLength = FileAr->TotalSize();
			const int64 BufferLength = Align(FileLength, PaddingAlignment);

			FileBuffer = FUniqueBuffer::Alloc((uint64)BufferLength);

			if (!ReadData(FileAr, FileBuffer.GetData(), FileLength, FilePath, OutErrors))
			{
				return;
			}

			if (OutInfo != nullptr)
			{
				FRehydrationInfo& Info = OutInfo->AddDefaulted_GetRef();
				Info.OriginalSize = FileLength;
				Info.RehydratedSize = FileLength;
				Info.NumPayloadsRehydrated = 0;
			}
		}

		OutPackages.Add(FileBuffer.MoveToShared());
	}

	// Make sure the arrays that we return have an entry per requested file
	check(PackagePaths.Num() == OutPackages.Num());
	check(OutInfo == nullptr || PackagePaths.Num() == OutInfo->Num());
}

} // namespace UE::Virtualization

#undef LOCTEXT_NAMESPACE
