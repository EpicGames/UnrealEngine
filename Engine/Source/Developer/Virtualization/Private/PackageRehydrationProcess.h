// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Internationalization/Text.h"

class FSharedBuffer;
struct FRehydrationInfo;

namespace UE::Virtualization
{

enum class ERehydrationOptions : uint32;
struct FRehydrationResult;

/** This version will rehydrate a package and replace it on disk */
void RehydratePackages(TConstArrayView<FString> PackagePaths, ERehydrationOptions Options, FRehydrationResult& OutResult);

/** This version will rehydrate a package but to a memory buffer, the original package on disk will remain untouched */
void RehydratePackages(TConstArrayView<FString> PackagePaths, uint64 PaddingAlignment, TArray<FText>& OutErrors, TArray<FSharedBuffer>& OutPackages, TArray<FRehydrationInfo>* OutInfo);

} // namespace UE::Virtualization
