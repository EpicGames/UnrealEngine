// Copyright Epic Games, Inc. All Rights Reserved.

#include "PackageUtils.h"

#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Internationalization/Internationalization.h"
#include "UObject/PackageTrailer.h"
#include "Misc/Paths.h"

#define LOCTEXT_NAMESPACE "Virtualization"

// When enabled we will validate truncated packages right after the truncation process to 
// make sure that the package format is still correct once the package trailer has been 
// removed.
#define UE_VALIDATE_TRUNCATED_PACKAGE 1

namespace UE::Virtualization
{

bool ValidatePackage(const FString& PackagePath, TArray<FText>& OutErrors)
{
	TUniquePtr<IFileHandle> TempFileHandle(FPlatformFileManager::Get().GetPlatformFile().OpenRead(*PackagePath));
	if (!TempFileHandle.IsValid())
	{
		FText ErrorMsg = FText::Format(LOCTEXT("Virtualization_OpenValidationFailed", "Unable to open '{0}' so that it can be validated"),
			FText::FromString(PackagePath));
		OutErrors.Add(ErrorMsg);

		return false;
	}

	TempFileHandle->SeekFromEnd(-4);

	uint32 PackageTag = INDEX_NONE;
	if (!TempFileHandle->Read((uint8*)&PackageTag, 4) || PackageTag != PACKAGE_FILE_TAG)
	{
		FText ErrorMsg = FText::Format(LOCTEXT("Virtualization_ValidationFailed", "The package '{0}' does not end with a valid tag, the file is considered corrupt"),
			FText::FromString(PackagePath));
		OutErrors.Add(ErrorMsg);

		return false;
	}


	return true;
}

bool CanWriteToFile(const FString& FilePath)
{
	TUniquePtr<FArchive> FileHandle(IFileManager::Get().CreateFileWriter(*FilePath, FILEWRITE_Append | FILEWRITE_Silent));

	return FileHandle.IsValid();
}

bool TryCopyPackageWithoutTrailer(const FString& SourcePath, const FString& DstPath, int64 TrailerLength, TArray<FText>& OutErrors)
{
	// TODO: Consider adding a custom copy routine to only copy the data we want, rather than copying the full file then truncating

	if (IFileManager::Get().Copy(*DstPath, *SourcePath) != ECopyResult::COPY_OK)
	{
		FText Message = FText::Format(LOCTEXT("Virtualization_CopyFailed", "Unable to copy package file '{0}' for virtualization"),
			FText::FromString(SourcePath));
		OutErrors.Add(Message);

		return false;
	}

	const int64 PackageSizeWithoutTrailer = IFileManager::Get().FileSize(*SourcePath) - TrailerLength;

	{
		TUniquePtr<IFileHandle> TempFileHandle(FPlatformFileManager::Get().GetPlatformFile().OpenWrite(*DstPath, true));
		if (!TempFileHandle.IsValid())
		{
			FText Message = FText::Format(LOCTEXT("Virtualization_TruncOpenFailed", "Failed to open package file for truncation'{0}' when virtualizing"),
				FText::FromString(DstPath));
			OutErrors.Add(Message);

			return false;
		}

		if (!TempFileHandle->Truncate(PackageSizeWithoutTrailer))
		{
			FText Message = FText::Format(LOCTEXT("Virtualization_TruncFailed", "Failed to truncate '{0}' when virtualizing"),
				FText::FromString(DstPath));
			OutErrors.Add(Message);

			return false;
		}
	}

#if UE_VALIDATE_TRUNCATED_PACKAGE
	// Validate that the package format of the new package is correct
	if (!ValidatePackage(DstPath, OutErrors))
	{
		return false;
	}
#endif //UE_VALIDATE_TRUNCATED_PACKAGE

	return true;
}

FString DuplicatePackageWithUpdatedTrailer(const FString& AbsolutePackagePath, const FPackageTrailer& Trailer, TArray<FText>& OutErrors)
{
	const FString BaseDir = FPaths::ProjectSavedDir() / TEXT("VADuplication");
	const FString BaseName = FPaths::GetBaseFilename(AbsolutePackagePath);
	const FString TempFilePath = FPaths::CreateTempFilename(*BaseDir, *BaseName.Left(32));

	// TODO Optimization: Combine TryCopyPackageWithoutTrailer with the appending of the new trailer to avoid opening multiple handles

	// Create copy of package minus the trailer the trailer
	if (!TryCopyPackageWithoutTrailer(AbsolutePackagePath, TempFilePath, Trailer.GetTrailerLength(), OutErrors))
	{
		return FString();
	}

	TUniquePtr<FArchive> PackageAr(IFileManager::Get().CreateFileReader(*AbsolutePackagePath));

	if (!PackageAr.IsValid())
	{
		FText Message = FText::Format(LOCTEXT("Virtualization_PkgOpen", "Failed to open the package '{1}' for reading"),
			FText::FromString(AbsolutePackagePath));
		OutErrors.Add(Message);

		return FString();
	}

	TUniquePtr<FArchive> CopyAr(IFileManager::Get().CreateFileWriter(*TempFilePath, EFileWrite::FILEWRITE_Append));
	if (!CopyAr.IsValid())
	{
		FText Message = FText::Format(LOCTEXT("Virtualization_TrailerAppendOpen", "Unable to open '{0}' to append the trailer'"),
			FText::FromString(TempFilePath));
		OutErrors.Add(Message);

		return FString();
	}

	FPackageTrailerBuilder TrailerBuilder = FPackageTrailerBuilder::CreateFromTrailer(Trailer, *PackageAr, AbsolutePackagePath);
	if (!TrailerBuilder.BuildAndAppendTrailer(nullptr, *CopyAr))
	{
		FText Message = FText::Format(LOCTEXT("Virtualization_TrailerAppend", "Failed to append the trailer to '{0}'"),
			FText::FromString(TempFilePath));
		OutErrors.Add(Message);

		return FString();
	}

	return TempFilePath;
}

FString DuplicatePackageWithNewTrailer(const FString& AbsolutePackagePath, const FPackageTrailer& OriginalTrailer, const FPackageTrailerBuilder& Builder, TArray<FText>& OutErrors)
{
	const FString BaseDir = FPaths::ProjectSavedDir() / TEXT("VADuplication");
	const FString BaseName = FPaths::GetBaseFilename(AbsolutePackagePath);
	const FString TempFilePath = FPaths::CreateTempFilename(*BaseDir, *BaseName.Left(32));

	// TODO Optimization: Combine TryCopyPackageWithoutTrailer with the appending of the new trailer to avoid opening multiple handles

	// Create copy of package minus the trailer the trailer
	if (!TryCopyPackageWithoutTrailer(AbsolutePackagePath, TempFilePath, OriginalTrailer.GetTrailerLength(), OutErrors))
	{
		return FString();
	}

	TUniquePtr<FArchive> CopyAr(IFileManager::Get().CreateFileWriter(*TempFilePath, EFileWrite::FILEWRITE_Append));
	if (!CopyAr.IsValid())
	{
		FText Message = FText::Format(LOCTEXT("Virtualization_TrailerAppendOpen", "Unable to open '{0}' to append the trailer'"),
			FText::FromString(TempFilePath));
		OutErrors.Add(Message);

		return FString();
	}

	FPackageTrailerBuilder CopyOfBuilder = Builder;
	if (!CopyOfBuilder.BuildAndAppendTrailer(nullptr, *CopyAr))
	{
		FText Message = FText::Format(LOCTEXT("Virtualization_TrailerAppend", "Failed to append the trailer to '{0}'"),
			FText::FromString(TempFilePath));
		OutErrors.Add(Message);

		return FString();
	}

	return TempFilePath;
}

} // namespace UE::Virtualization

#undef LOCTEXT_NAMESPACE
