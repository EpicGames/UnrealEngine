// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"

namespace UE
{

class FPackageTrailer;
class FPackageTrailerBuilder;

} // namespace UE

namespace UE::Virtualization
{

/**
 * Check that the given package ends with PACKAGE_FILE_TAG. Intended to be used to make sure that
 * we have truncated a package correctly when removing the trailers.
 *
 * @param PackagePath	The path of the package that should be checked
 * @param Errors [out] 	Errors created by the function will be added here
 *
 * @return	True if the package is correctly terminated with a PACKAGE_FILE_TAG, false if the tag
 *			was not found or if we were unable to read the file's contents.
 */
bool ValidatePackage(const FString& PackagePath, TArray<FText>& OutErrors);

/** Tests if we would be able to write to the given file if we wanted to */
bool CanWriteToFile(const FString& FilePath);

// TODO: The following functions probably need reworking to be safer/faster
// for example passing in the original trailer in order to remove it from the original 
// package can be error prone if the wrong trailer is supplied.

/**
 * Creates a copy of the given package but the copy will not include the FPackageTrailer.
 *
 * @param SourcePath	The absolute path of the package to copy
 * @param DstPath		The path where the copy should be created
 * @param TrailerLength	The length of the trailer to be removed
 * @param Errors [out]	Errors created by the function will be added here
 *
 * @return Returns true if the package was copied correctly, false otherwise. Note even when returning false a file might have been created at 'CopyPath'
 */
bool TryCopyPackageWithoutTrailer(const FString& SourcePath, const FString& DstPath, int64 TrailerLength, TArray<FText>& OutErrors);

/**
 * Create a copy of an existing package but replace the trailer with an updated version of the original trailer. The copy will be to a tmp file
 * under the 'Saved' directory.
 * 
 * @param AbsolutePackagePath	The path of the package to duplicate
 * @param Tailer				The original trailer in the package, which has been updated with calls to UpdatePayloadAsVirtualized
 * @param OutErrors				An array that will contain any errors encountered
 * 
 * @return The path of the duplicated package file, if the duplicate was not created due to errors then the path returns will be empty
 */
FString DuplicatePackageWithUpdatedTrailer(const FString& AbsolutePackagePath, const FPackageTrailer& Trailer, TArray<FText>& OutErrors);

/**
 * Create a copy of an existing package but replace the trailer with an updated version of the original trailer. The copy will be to a tmp file
 * under the 'Saved' directory.
 *
 * @param AbsolutePackagePath	The path of the package to duplicate
 * @param Tailer				The original trailer in the package
 * @param Builder				A builder filled with the data required to make the new package trailer
 * @param OutErrors				An array that will contain any errors encountered
 *
 * @return The path of the duplicated package file, if the duplicate was not created due to errors then the path returns will be empty
 */
FString DuplicatePackageWithNewTrailer(const FString& AbsolutePackagePath, const FPackageTrailer& Trailer, const FPackageTrailerBuilder& Builder, TArray<FText>& OutErrors);



} // namespace UE::Virtualization
