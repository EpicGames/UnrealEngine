// Copyright Epic Games, Inc. All Rights Reserved.

#include "PackageVirtualizationProcess.h"

#include "Containers/UnrealString.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformTime.h"
#include "Internationalization/Internationalization.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Misc/ScopeExit.h"
#include "Misc/ScopedSlowTask.h"
#include "PackageUtils.h"
#include "Serialization/EditorBulkData.h"
#include "UObject/Linker.h"
#include "UObject/Package.h"
#include "UObject/PackageResourceManager.h"
#include "UObject/PackageTrailer.h"
#include "UObject/UObjectGlobals.h"
#include "Virtualization/VirtualizationSystem.h"
#include "VirtualizationManager.h"
#include "VirtualizationSourceControlUtilities.h"
#include "VirtualizationUtilities.h"

#define LOCTEXT_NAMESPACE "Virtualization"

namespace UE::Virtualization
{

/** 
 * Implementation of the IPayloadProvider interface so that payloads can be requested on demand
 * when they are being virtualized.
 * 
 * This implementation is not optimized. If a package holds many payloads that are all virtualized
 * we will end up loading the same trailer over and over, as well as opening the same package file
 * for read many times.
 * 
 * So far this has shown to be a rounding error compared to the actual cost of virtualization 
 * and so implementing any level of caching has been left as a future task.
 * 
 * TODO: Implement a MRU cache for payloads to prevent loading the same payload off disk many
 * times for different backends if it will not cause a huge memory spike.
 */
class FWorkspaceDomainPayloadProvider final : public IPayloadProvider
{
public: 
	FWorkspaceDomainPayloadProvider() = default;
	virtual ~FWorkspaceDomainPayloadProvider() = default;

	/** Register the payload with it's trailer and package name so that we can access it later as needed */
	void RegisterPayload(const FIoHash& PayloadId, uint64 SizeOnDisk, const FString& PackageName)
	{
		if (!PayloadId.IsZero())
		{
			PayloadLookupTable.Emplace(PayloadId, FPayloadData(SizeOnDisk, PackageName));
		}
	}

private:
	virtual FCompressedBuffer RequestPayload(const FIoHash& Identifier) override
	{
		if (Identifier.IsZero())
		{
			return FCompressedBuffer();
		}

		const FPayloadData* Data = PayloadLookupTable.Find(Identifier);
		if (Data == nullptr)
		{
			UE_LOGF(LogVirtualization, Error, "FWorkspaceDomainPayloadProvider was unable to find a payload with the identifier '%ls'", 
				*LexToString(Identifier));

			return FCompressedBuffer();
		}
		
		TUniquePtr<FArchive> PackageAr = IPackageResourceManager::Get().OpenReadExternalResource(EPackageExternalResource::WorkspaceDomainFile, *Data->PackageName);

		if (!PackageAr.IsValid())
		{
			UE_LOGF(LogVirtualization, Error, "FWorkspaceDomainPayloadProvider was unable to open the package '%ls' for reading", 
				*Data->PackageName);

			return FCompressedBuffer();
		}
			
		PackageAr->Seek(PackageAr->TotalSize());

		FPackageTrailer Trailer;
		if (!Trailer.TryLoadBackwards(*PackageAr))
		{
			UE_LOGF(LogVirtualization, Error, "FWorkspaceDomainPayloadProvider failed to load the package trailer from the package '%ls'", 
				*Data->PackageName);

			return FCompressedBuffer();
		}

		FCompressedBuffer Payload = Trailer.LoadLocalPayload(Identifier, *PackageAr);
		
		if (!Payload)
		{
			UE_LOGF(LogVirtualization, Error, "FWorkspaceDomainPayloadProvider was uanble to load the payload '%ls' from the package '%ls'",
				*LexToString(Identifier),
				*Data->PackageName);

			return FCompressedBuffer();
		}

		if (Identifier != FIoHash(Payload.GetRawHash()))
		{
			UE_LOGF(LogVirtualization, Error, "FWorkspaceDomainPayloadProvider loaded an incorrect payload from the package '%ls'. Expected '%ls' Loaded  '%ls'", 
				*Data->PackageName,
				*LexToString(Identifier),
				*LexToString(Payload.GetRawHash()));

			return FCompressedBuffer();
		}

		return Payload;
	}

	virtual uint64 GetPayloadSize(const FIoHash& Identifier) override
	{
		if (Identifier.IsZero())
		{
			return 0;
		}

		const FPayloadData* Data = PayloadLookupTable.Find(Identifier);
		if (Data != nullptr)
		{
			return Data->SizeOnDisk;
		}
		else
		{
			UE_LOGF(LogVirtualization, Error, "FWorkspaceDomainPayloadProvider was unable to find a payload with the identifier '%ls'",
				*LexToString(Identifier));

			return 0;
		}
	}

	/* This structure holds additional info about the payload that we might need later */
	struct FPayloadData
	{
		FPayloadData(uint64 InSizeOnDisk, const FString& InPackageName)
			: SizeOnDisk(InSizeOnDisk)
			, PackageName(InPackageName)
		{

		}

		uint64 SizeOnDisk;
		FString PackageName;
	};

	TMap<FIoHash, FPayloadData> PayloadLookupTable;
};

void VirtualizePackages(TConstArrayView<FString> PackagePaths, EVirtualizationOptions Options, FVirtualizationResult& OutResultInfo)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::Virtualization::VirtualizePackages);

	IVirtualizationSystem& System = IVirtualizationSystem::Get();

	const double StartTime = FPlatformTime::Seconds();
	
	ON_SCOPE_EXIT
	{
		OutResultInfo.TimeTaken = FPlatformTime::Seconds() - StartTime;
		UE_LOGF(LogVirtualization, Display, "Virtualization process complete");
		UE_LOGF(LogVirtualization, Verbose, "Virtualization process took %.3f(s)", OutResultInfo.TimeTaken);
	};

	FScopedSlowTask Progress(5.0f, LOCTEXT("Virtualization_Task", "Virtualizing Assets..."));
	// Force the task to be visible otherwise it might not be shown if the initial progress frames are too fast
	Progress.Visibility = ESlowTaskVisibility::ForceVisible;
	Progress.MakeDialog();

	// Other systems may have added errors to this array, we need to check so later we can determine if this function added any additional errors.
	const int32 NumErrors = OutResultInfo.GetNumErrors();

	struct FPackageInfo
	{
		FPackagePath Path;
		FPackageTrailer Trailer;

		TArray<FIoHash> LocalPayloads;

		/** Index where the FPushRequest for this package can be found */
		int32 PayloadIndex = INDEX_NONE;

		bool bWasTrailerUpdated = false;
	};

	UE_LOGF(LogVirtualization, Display, "Considering %d file(s) for virtualization", PackagePaths.Num());

	TArray<FPackageInfo> Packages;
	Packages.Reserve(PackagePaths.Num());

	Progress.EnterProgressFrame(1.0f);

	// From the list of files to submit we need to find all of the valid packages that contain
	// local payloads that need to be virtualized.
	int64 TotalPackagesFound = 0; 
	int64 TotalOutOfDatePackages = 0;
	int64 TotalPackageTrailersFound = 0;
	int64 TotalPayloadsToVirtualize = 0;

	TSet<FString> ConsideredPackages;
	ConsideredPackages.Reserve(PackagePaths.Num());

	for (const FString& AbsoluteFilePath : PackagePaths)
	{
		FPackagePath PackagePath = FPackagePath::FromLocalPath(AbsoluteFilePath);

		bool bIsDuplicate = false;
		ConsideredPackages.Add(PackagePath.GetPackageName(), &bIsDuplicate);
		if (bIsDuplicate)
		{
			UE_LOGF(LogVirtualization, Verbose, "Skipping duplicate package entry '%ls'", *AbsoluteFilePath);
			continue; // Skip duplicate packages
		}

		// TODO: How to handle text packages?
		if (FPackageName::IsPackageExtension(PackagePath.GetHeaderExtension()) || FPackageName::IsTextPackageExtension(PackagePath.GetHeaderExtension()))
		{
			TotalPackagesFound++;

			FPackageTrailer Trailer;
			if (FPackageTrailer::TryLoadFromPackage(PackagePath, Trailer))
			{
				TotalPackageTrailersFound++;

				// The following is not expected to ever happen, currently we give a user facing error but it generally means that the asset is broken somehow.
				ensureMsgf(Trailer.GetNumPayloads(EPayloadStorageType::Referenced) == 0, TEXT("Trying to virtualize a package that already contains payload references which the workspace file should not ever contain!"));
				if (Trailer.GetNumPayloads(EPayloadStorageType::Referenced) > 0)
				{
					FText Message = FText::Format(LOCTEXT("Virtualization_PkgHasReferences", "Cannot virtualize the package '{1}' as it has referenced payloads in the trailer"),
						FText::FromString(PackagePath.GetDebugName()));
					OutResultInfo.AddError(MoveTemp(Message));
					return;
				}

				FPackageInfo PkgInfo;

				PkgInfo.Path = MoveTemp(PackagePath);
				PkgInfo.Trailer = MoveTemp(Trailer);
				PkgInfo.LocalPayloads = PkgInfo.Trailer.GetPayloads(EPayloadFilter::CanVirtualize);

				if (!PkgInfo.LocalPayloads.IsEmpty())
				{
					TotalPayloadsToVirtualize += PkgInfo.LocalPayloads.Num();

					Packages.Emplace(MoveTemp(PkgInfo));
				}
			}
			else if(Utils::FindTrailerFailedReason(PackagePath) == Utils::ETrailerFailedReason::OutOfDate)
			{
				
				TotalOutOfDatePackages++;
			}
		}
	}

	const int32 NumSkippedPackages = PackagePaths.Num() - ConsideredPackages.Num();
	ConsideredPackages.Empty();

	UE_LOG(LogVirtualization, Display, TEXT("Found %" INT64_FMT " package(s), %" INT64_FMT " of which had payload trailers"), TotalPackagesFound, TotalPackageTrailersFound);
	UE_CLOGF(NumSkippedPackages > 0, LogVirtualization, Warning, "Discarded %d duplicate package paths", NumSkippedPackages);
	UE_CLOG(TotalOutOfDatePackages > 0, LogVirtualization, Warning, TEXT("Found %" INT64_FMT " package(s) that are out of date and need resaving"), TotalOutOfDatePackages);

	// TODO: Currently not all of the filtering is done as package save time, so some of the local payloads may not get virtualized.
	// When/if we move all filtering to package save we can change this log message to state that the local payloads *will* be virtualized.
	UE_LOG(LogVirtualization, Display, TEXT("Found %" INT64_FMT " locally stored payload(s) in %d package(s) that maybe need to be virtualized"), TotalPayloadsToVirtualize, Packages.Num());

	Progress.EnterProgressFrame(1.0f);

	// TODO Optimization: We might want to check for duplicate payloads and remove them at this point

	// Build up the info in the payload provider and the final array of payload push requests
	FWorkspaceDomainPayloadProvider PayloadProvider;
	TArray<Virtualization::FPushRequest> PayloadsToSubmit;
	PayloadsToSubmit.Reserve( IntCastChecked<int32>(TotalPayloadsToVirtualize) );

	for (FPackageInfo& PackageInfo : Packages)
	{
		check(!PackageInfo.LocalPayloads.IsEmpty());

		PackageInfo.PayloadIndex = PayloadsToSubmit.Num();

		for (const FIoHash& PayloadId : PackageInfo.LocalPayloads)
		{
			const uint64 SizeOnDisk = PackageInfo.Trailer.FindPayloadSizeOnDisk(PayloadId);

			PayloadProvider.RegisterPayload(PayloadId, SizeOnDisk, PackageInfo.Path.GetPackageName());
			PayloadsToSubmit.Emplace(PayloadId, PayloadProvider, PackageInfo.Path.GetPackageName());
		}
	}

	// TODO: We should be able to do both Cache and Persistent pushes in the same call

	// Push payloads to cache storage
	Progress.EnterProgressFrame(1.0f);
	if(System.IsPushingEnabled(EStorageType::Cache))
	{
		UE_LOGF(LogVirtualization, Display, "Pushing payload(s) to EStorageType::Cache storage...");

		if (!System.PushData(PayloadsToSubmit, EStorageType::Cache))
		{
			// Caching is not critical to the process so we only warn if it fails
			UE_LOGF(LogVirtualization, Warning, "Failed to push to EStorageType::Cache storage");
		}

		int64 TotalPayloadsCached = 0;
		for (Virtualization::FPushRequest& Request : PayloadsToSubmit)
		{
			TotalPayloadsCached += Request.GetResult().WasPushed() ? 1 : 0;

			// TODO: This really shouldn't be required, fix when we allow both pushes to be done in the same call
			// Reset the status for the persistent storage push
			Request.ResetResult();
		}
		UE_LOG(LogVirtualization, Display, TEXT("Pushed %" INT64_FMT " payload(s) to cached storage"), TotalPayloadsCached);
	}
	else
	{
		UE_LOGF(LogVirtualization, Display, "Pushing payload(s) to cached storage is disbled, skipping");
	}
	
	// Push payloads to persistent storage
	{
		Progress.EnterProgressFrame(1.0f);

		UE_LOGF(LogVirtualization, Display, "Pushing payload(s) to EStorageType::Persistent storage...");

		if (!System.PushData(PayloadsToSubmit, EStorageType::Persistent))
		{
			FText Message = LOCTEXT("Virtualization_PushFailure", "Failed to push payloads");
			OutResultInfo.AddError(MoveTemp(Message));
			return;
		}

		int64 TotalPayloadsVirtualized = 0;
		for (const Virtualization::FPushRequest& Request : PayloadsToSubmit)
		{
			TotalPayloadsVirtualized += Request.GetResult().WasPushed() ? 1 : 0;
		}
		UE_LOG(LogVirtualization, Display, TEXT("Pushed %" INT64_FMT " payload(s) to EStorageType::Persistent storage"), TotalPayloadsVirtualized);
	}

	// Update the package info for the submitted payloads
	for (FPackageInfo& PackageInfo : Packages)
	{
		for (int32 Index = 0; Index < PackageInfo.LocalPayloads.Num(); ++Index)
		{
			const Virtualization::FPushRequest& Request = PayloadsToSubmit[PackageInfo.PayloadIndex + Index];
			check(Request.GetIdentifier() == PackageInfo.LocalPayloads[Index]);

			if (Request.GetResult().IsVirtualized())
			{
				if (PackageInfo.Trailer.UpdatePayloadAsVirtualized(Request.GetIdentifier()))
				{
					PackageInfo.bWasTrailerUpdated = true;
				}
				else
				{
					FText Message = FText::Format(	LOCTEXT("Virtualization_UpdateStatusFailed", "Unable to update the status for the payload '{0}' in the package '{1}'"),
													FText::FromString(LexToString(Request.GetIdentifier())),
													FText::FromString(PackageInfo.Path.GetDebugName()));
					OutResultInfo.AddError(MoveTemp(Message));
					return;
				}
			}
		}
	}

	// If the process has created new errors by this point we should early out before we actually start making changes on disk
	if (NumErrors != OutResultInfo.GetNumErrors())
	{
		return;
	}

	Progress.EnterProgressFrame(1.0f);

	struct FPackageReplacement
	{
		FPackagePath Path;
		FPackageTrailer Trailer;
	};

	TArray<FPackageReplacement> PackagesToReplace;
	PackagesToReplace.Reserve(Packages.Num());

	for (FPackageInfo& PackageInfo : Packages)
	{
		if (PackageInfo.bWasTrailerUpdated)
		{
			PackagesToReplace.Add({MoveTemp(PackageInfo.Path), MoveTemp(PackageInfo.Trailer)});
		}
	}
	Packages.Empty(); // No longer used, the useful data have been moved to PackagesToReplace

	if (PackagesToReplace.IsEmpty())
	{
		UE_LOGF(LogVirtualization, Display, "No packages need to be updated on disk");
		return;
	}

	UE_LOGF(LogVirtualization, Display, "%d package(s) had their trailer container modified and need to be updated", PackagesToReplace.Num());

	{
		// We need to reset the loader of any loaded package that we want save over on disk so that
		// the file lock is relinquished

		TRACE_CPUPROFILER_EVENT_SCOPE(UE::Virtualization::VirtualizePackages::ResetLoaders);

		UE_LOGF(LogVirtualization, Display, "Detaching loaded packages from disk...");

		int32 NumPackagesReset = 0;
		for (const FPackageReplacement& PackageInfo : PackagesToReplace)
		{
			UPackage* LoadedPackage = FindObjectFast<UPackage>(nullptr, PackageInfo.Path.GetPackageFName());
			if (LoadedPackage != nullptr)
			{
				UE_LOGF(LogVirtualization, Verbose, "Detaching '%ls'", *PackageInfo.Path.GetDebugName());
				// TODO: Consider using the batch API
				ResetLoadersForSave(LoadedPackage, *PackageInfo.Path.GetLocalFullPath());
				NumPackagesReset++;
			}
		}

		UE_LOGF(LogVirtualization, Display, "Reset the loaders of %d package(s)", NumPackagesReset);
	}

	if (EnumHasAnyFlags(Options, EVirtualizationOptions::Checkout))
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::Virtualization::VirtualizePackages::Checkout);

		UE_LOGF(LogVirtualization, Display, "Checking out packages from revision control...");

		TArray<FString> FilesToCheckState;
		FilesToCheckState.Reserve(PackagesToReplace.Num());

		for (const FPackageReplacement& PackageInfo : PackagesToReplace)
		{
			FilesToCheckState.Add(PackageInfo.Path.GetLocalFullPath());
		}

		if (!TryCheckoutFiles(FilesToCheckState, OutResultInfo.Errors, &OutResultInfo.CheckedOutPackages))
		{
			return;
		}

		UE_LOGF(LogVirtualization, Display, "Checked out %d package(s)", OutResultInfo.CheckedOutPackages.Num());
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::Virtualization::VirtualizePackages::EnsureWritePermissions);

		UE_CLOGF(!PackagesToReplace.IsEmpty(), LogVirtualization, Display, "Checking packages for write access permission...");

		// Now check to see if there are package files that cannot be edited because they are read only
		int32 NumSkipped = 0;
		for (int32 Index = 0; Index < PackagesToReplace.Num(); ++Index)
		{
			const FPackageReplacement& PackageInfo = PackagesToReplace[Index];

			if (!CanWriteToFile(PackageInfo.Path.GetLocalFullPath()))
			{
				// Technically the package could have local payloads that won't be virtualized due to filtering or min payload sizes and so the
				// following warning is misleading. This will be solved if we move that evaluation to the point of saving a package.
				// If not then we probably need to extend QueryPayloadStatuses to test filtering etc as well, then check for potential package
				// modification after that.
				// Long term, the stand alone tool should be able to request the UnrealEditor relinquish the lock on the package file so this becomes 
				// less of a problem.
				FText Message = FText::Format(LOCTEXT("Virtualization_PkgLocked", "The package file '{0}' has local payloads but is locked for modification and cannot be virtualized, this package will be skipped!"),
					FText::FromString(PackageInfo.Path.GetDebugName()));

				UE_LOGF(LogVirtualization, Warning, "%ls", *Message.ToString());

				PackagesToReplace.RemoveAt(Index--);
				NumSkipped++;
			}
		}

		UE_CLOGF(NumSkipped > 0, LogVirtualization, Warning, "Skipped %d package(s)", NumSkipped);
		UE_CLOGF(NumSkipped == 0, LogVirtualization, Display, "All packages have write permission");
	}
	
	{
		// Since we had no errors we can now replace all of the packages that were virtualized data with the virtualized replacement file.
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::Virtualization::VirtualizePackages::ReplacePackages);

		UE_LOGF(LogVirtualization, Display, "Replacing old packages with the virtualized version...");

		OutResultInfo.PreOperationSize = 0;
		OutResultInfo.PostOperationSize = 0;

		for (const FPackageReplacement& PackageInfo : PackagesToReplace)
		{
			const FString OriginalPackagePath = PackageInfo.Path.GetLocalFullPath();
			const FString NewPackagePath = DuplicatePackageWithUpdatedTrailer(OriginalPackagePath, PackageInfo.Trailer, OutResultInfo.Errors);

			if (NewPackagePath.IsEmpty())
			{
				// Duplication failed so skip this package for now. The error will be in OutResultInfo.Errors
				continue;
			}

			const int64 OriginalSize = IFileManager::Get().GetStatData(*OriginalPackagePath).FileSize;
			const int64 ReducedSize = IFileManager::Get().GetStatData(*NewPackagePath).FileSize;

			if (IFileManager::Get().Move(*OriginalPackagePath, *NewPackagePath))
			{
				OutResultInfo.VirtualizedPackages.Add(OriginalPackagePath);

				OutResultInfo.PreOperationSize += OriginalSize;
				OutResultInfo.PostOperationSize += ReducedSize;

				UE_LOGF(LogVirtualization, Verbose, "Reducing %ls: %ls -> %ls",
					*PackageInfo.Path.GetDebugName(),
					*FText::AsMemory(OriginalSize).ToString(),
					*FText::AsMemory(ReducedSize).ToString());
			}
			else
			{
				FText Message = FText::Format(LOCTEXT("Virtualization_MoveFailed", "Unable to replace the package '{0}' with the virtualized version"),
					FText::FromString(PackageInfo.Path.GetDebugName()));

				OutResultInfo.AddError(MoveTemp(Message));
				continue;
			}
		}

		if (OutResultInfo.PreOperationSize != OutResultInfo.PostOperationSize)
		{
			UE_LOGF(LogVirtualization, Display, "Total Reduction %ls (%ls -> %ls)",
				*FText::AsMemory(OutResultInfo.PreOperationSize - OutResultInfo.PostOperationSize).ToString(),
				*FText::AsMemory(OutResultInfo.PreOperationSize).ToString(),
				*FText::AsMemory(OutResultInfo.PostOperationSize).ToString());
		}

		UE_LOGF(LogVirtualization, Display, "Replaced %d package(s)", OutResultInfo.VirtualizedPackages.Num());
	}
}

} // namespace UE::Virtualization

#undef LOCTEXT_NAMESPACE

