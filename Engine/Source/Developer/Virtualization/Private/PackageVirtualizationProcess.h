// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Internationalization/Text.h"

namespace UE::Virtualization
{

enum class EVirtualizationOptions : uint32;
struct FVirtualizationResult;

void VirtualizePackages(TConstArrayView<FString> PackagePaths, EVirtualizationOptions Options, FVirtualizationResult& OutResultInfo);

} // namespace UE::Virtualization
