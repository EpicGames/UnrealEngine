// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVirtualizationRevisionControlConnectionDialog.h"

#if UE_VA_WITH_SLATE

#include "Async/ManualResetEvent.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "HAL/IConsoleManager.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "SourceControlHelpers.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "VirtualizationManager.h"

#define LOCTEXT_NAMESPACE "Virtualization"

namespace UE::Virtualization
{

SRevisionControlConnectionDialog::FResult SRevisionControlConnectionDialog::RunDialog(FStringView RevisionControlName, FStringView ConfigSectionName,  FStringView CurrentPort, FStringView CurrentUsername, const FText& ErrorMessage)
{
	if (FApp::IsUnattended())
	{
		UE_LOGF(LogVirtualization, Warning, "Skipping attempt to show SRevisionControlConnectionDialog as the application is unattended");
		return FResult();
	}

	if (!IsInGameThread())
	{
		UE_LOGF(LogVirtualization, Warning, "Attempting to show SRevisionControlConnectionDialog on a worker thread!");
		return FResult();
	}

	if (!FSlateApplication::IsInitialized() || FSlateApplication::Get().GetRenderer() == nullptr)
	{
		UE_LOGF(LogVirtualization, Warning, "Attempting to show SRevisionControlConnectionDialog before slate is initialized");
		return FResult();
	}

	UE_LOGF(LogVirtualization, Display, "Creating dialog");

	FText WindowTitle = FText::Format(LOCTEXT("VASCSettings", "Virtualized Assets - {0} Revision Control Settings"), FText::FromStringView(RevisionControlName));

	TSharedPtr<SWindow> DialogWindow = SNew(SWindow)
		.Title(MoveTemp(WindowTitle))
		.FocusWhenFirstShown(true)
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		.SizingRule(ESizingRule::Autosized)
		.HasCloseButton(false);

	TSharedPtr<SRevisionControlConnectionDialog> DialogWidget;

	TSharedPtr<SBorder> DialogWrapper =
		SNew(SBorder)
		.Padding(4.0f)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(16.0f, 16.0f, 16.0f, 0.0f)
			[
				SAssignNew(DialogWidget, SRevisionControlConnectionDialog, RevisionControlName, ConfigSectionName, CurrentPort, CurrentUsername, ErrorMessage)
				.Window(DialogWindow)
			]
		];

	DialogWindow->SetContent(DialogWrapper.ToSharedRef());

	UE_LOGF(LogVirtualization, Display, "Connection to revision control for virtualized assets failed. Offering the user the choice to retry or continue anyway");

	TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().GetActiveTopLevelWindow();
	FSlateApplication::Get().AddModalWindow(DialogWindow.ToSharedRef(), ParentWindow);

	switch (DialogWidget->GetResult())
	{
		case EResult::Retry:
			return FResult(DialogWidget->GetPort(), DialogWidget->GetUserName());
		case EResult::Skip:
			return FResult();
		default:
			checkNoEntry();
			return FResult();
	}
}

void SRevisionControlConnectionDialog::Construct(const FArguments& InArgs, FStringView RevisionControlName, FStringView InConfigSectionName, FStringView CurrentPort, FStringView CurrentUsername, const FText& ErrorMessage)
{
	WindowWidget = InArgs._Window;

	const FString ConnectionHelpUrl = FVirtualizationManager::GetConnectionHelpUrl();
	ConfigSectionName = InConfigSectionName;

	FText MessagePt1 = FText::Format(LOCTEXT("VASCMsgPt1", "Failed to connect to the {0} revision control server with the following errors:"), FText::FromStringView(RevisionControlName));
	FText MessagePt2 = FText::Format(LOCTEXT("VASCMsgPt2", "This may prevent you from loading virtualized assets in the future!\nPlease enter the correct {0} revision control settings below:"), FText::FromStringView(RevisionControlName));
	
	const FText PortToolTip = FText::Format(LOCTEXT("PortLabel_Tooltip", "The server and port for your {0} server. Usage ServerName:1234."), FText::FromStringView(RevisionControlName));
	const FText UserToolTip = FText::Format(LOCTEXT("UserNameLabel_Tooltip", "{0} username."), FText::FromStringView(RevisionControlName));

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(FMargin(0.0f, 0.0f, 16.0f, 16.0f))
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(FMargin(0.0f, 0.0f, 8.0f, 0.0f))
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.Padding(FMargin(0.0f, 0.0f, 0.0f, 0.0f))
				.AutoHeight()
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.Error"))
				]
			]
			+ SHorizontalBox::Slot()
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.Padding(FMargin(0.0f, 0.0f, 0.0f, 0.0f))
				.AutoHeight()
				[
					SNew(STextBlock)
					.Text(MoveTemp(MessagePt1))
					.AutoWrapText(true)
				]
				+ SVerticalBox::Slot()
				.Padding(FMargin(0.0f, 16.0f, 0.0f, 16.0f))
				.AutoHeight()
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::Get().GetBrush("BlackBrush"))
					[
						SNew(STextBlock)
						.ColorAndOpacity(FSlateColor(EStyleColor::Error))
						.Text(ErrorMessage)
						.AutoWrapText(true)
					]
				]
				+ SVerticalBox::Slot()
				.Padding(FMargin(0.0f, 0.0f, 0.0f, 0.0f))
				.AutoHeight()
				[
					SNew(STextBlock)
					.Text(MoveTemp(MessagePt2))
					.AutoWrapText(true)
				]
				
			]
		]
		+ SVerticalBox::Slot()
		.Padding(FMargin(0.0f, 0.0f, 16.0f, 16.0f))
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(0.2f)
			.Padding(FMargin(0.0f, 0.0f, 16.0f, 0.0f))
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.Padding(FMargin(0.0f, 0.0f, 0.0f, 10.0f))
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("PortLabel", "Server"))
					.ToolTipText(PortToolTip)
				]
				+ SVerticalBox::Slot()
				.Padding(FMargin(0.0f, 0.0f, 0.0f, 10.0f))
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("UserNameLabel", "User Name"))
					.ToolTipText(UserToolTip)
				]
			]
			+ SHorizontalBox::Slot()
			.FillWidth(0.8f)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(FMargin(0.0f, 0.0f, 0.0f, 10.0f))
				[
					SAssignNew(PortTextWidget, SEditableTextBox)
					.Text(FText::FromString(FString(CurrentPort)))
					.ToolTipText(PortToolTip)
				]
				+ SVerticalBox::Slot()
				.Padding(FMargin(0.0f, 0.0f, 0.0f, 10.0f))
				.VAlign(VAlign_Center)
				[
					SAssignNew(UsernameTextWidget, SEditableTextBox)
					.Text(FText::FromString(FString(CurrentUsername)))
					.ToolTipText(UserToolTip)
				]
			]
		]
		+ SVerticalBox::Slot()
		.Padding(FMargin(0.0f, 0.0f, 16.0f, 16.0f))
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(FMargin(0.0f, 0.0f, 8.0f, 0.0f))
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.Padding(FMargin(0.0f, 0.0f, 0.0f, 0.0f))
				.AutoHeight()
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.Warning"))
				]
			]
			+ SHorizontalBox::Slot()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("VASCSkipWarning", "Skipping may cause future editor instability if virtualized data is required!"))
				.AutoWrapText(true)
			]
		]
		+ SVerticalBox::Slot()
		.HAlign(HAlign_Right)
		.AutoHeight()
		[
			SNew(SHyperlink)
			.Text(LOCTEXT("VASCHelpUrl", "Click here for additional documentation"))
			.ToolTipText(FText::FromString(ConnectionHelpUrl))
			.OnNavigate(this, &SRevisionControlConnectionDialog::OnUrlClicked)
			.Visibility_Lambda([ConnectionHelpUrl] { return !ConnectionHelpUrl.IsEmpty() ? EVisibility::Visible : EVisibility::Hidden; })
		]
		+ SVerticalBox::Slot()
		.Padding(FMargin(0.0f, 16.0f, 16.0f, 16.0f))
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			[
				SNew(SButton)
				.VAlign(VAlign_Center)
				.TextStyle(FAppStyle::Get(), "DialogButtonText")
				.Text(LOCTEXT("VASC_Reset", "Reset To Defaults"))
				.ToolTipText(LOCTEXT("VASC_ResetTip", "Removes connection settings that may be saved to your local ini files and attempts to connect using your environment defaults"))
				.OnClicked(this, &SRevisionControlConnectionDialog::OnResetToDefaults)
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.VAlign(VAlign_Center)
					.ButtonStyle(FAppStyle::Get(), "PrimaryButton")
					.TextStyle(FAppStyle::Get(), "DialogButtonText")
					.Text(LOCTEXT("VASC_Retry", "Retry Connection"))
					.ToolTipText(LOCTEXT("VASC_RetryTip", "Attempts to reconnect to the revision control server with the settings that you entered"))
					.OnClicked(this, &SRevisionControlConnectionDialog::OnRetryConnection)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(10.0f, 0.0f, 0.0f, 0.0f))
				[
					SNew(SButton)
					.VAlign(VAlign_Center)
					.TextStyle(FAppStyle::Get(), "DialogButtonText")
					.Text(LOCTEXT("VASC_Skip", "Skip"))
					.ToolTipText(LOCTEXT("VASC_SkipTip", "The editor will continue to load but will be unable to pull virtualized data from revision control if needed"))
					.OnClicked(this, &SRevisionControlConnectionDialog::OnSkip)
				]
			]
		]
	];
}

void SRevisionControlConnectionDialog::CloseModalDialog()
{
	if (WindowWidget.IsValid())
	{
		WindowWidget.Pin()->RequestDestroyWindow();
	}
}

FReply SRevisionControlConnectionDialog::OnResetToDefaults()
{
	UE_LOGF(LogVirtualization, Display, "User opted to clear the ini file settings and connect using the revision control defaults");
	GConfig->EmptySection(*ConfigSectionName, SourceControlHelpers::GetSettingsIni());

	Result = EResult::Retry;

	Port.Empty();
	UserName.Empty();

	CloseModalDialog();

	return FReply::Handled();
}

FReply SRevisionControlConnectionDialog::OnRetryConnection()
{
	UE_LOGF(LogVirtualization, Display, "User opted to retry connecting to revision control");

	Result = EResult::Retry;

	Port = PortTextWidget->GetText().ToString();
	UserName = UsernameTextWidget->GetText().ToString();

	CloseModalDialog();

	return FReply::Handled();
}

FReply SRevisionControlConnectionDialog::OnSkip()
{
	UE_LOGF(LogVirtualization, Warning, "User opted not to connect to revision control. Virtualized data may not be accessible!");

	Result = EResult::Skip;

	CloseModalDialog();

	return FReply::Handled();
}

void SRevisionControlConnectionDialog::OnUrlClicked() const
{
	const FString ConnectionHelpUrl = FVirtualizationManager::GetConnectionHelpUrl();

	FPlatformProcess::LaunchURL(*ConnectionHelpUrl, nullptr, nullptr);
}

} // namespace UE::Virtualization

#undef LOCTEXT_NAMESPACE

#endif //UE_VA_WITH_SLATE
