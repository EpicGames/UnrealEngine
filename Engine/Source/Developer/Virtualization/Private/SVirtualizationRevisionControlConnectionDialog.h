// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_VA_WITH_SLATE

#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"

class SEditableTextBox;
class SWindow;

namespace UE::Virtualization
{

class SRevisionControlConnectionDialog : public SCompoundWidget
{
public:
	struct FResult
	{
	public:
		FResult() = default;
		FResult(const FString& InPort, const FString& InUserName)
			: bShouldRetry(true)
			, Port(InPort)
			, UserName(InUserName)
		{

		}

		~FResult() = default;

	public:
		bool bShouldRetry = false;

		FString Port;
		FString UserName;
	};

	static FResult RunDialog(FStringView RevisionControlName, FStringView ConfigSectionName, FStringView CurrentPort, FStringView CurrentUsername, const FText& ErrorMessage);

	SLATE_BEGIN_ARGS(SRevisionControlConnectionDialog) {}
		SLATE_ARGUMENT(TSharedPtr<SWindow>, Window)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FStringView RevisionControlName, FStringView ConfigSectionName, FStringView CurrentPort, FStringView CurrentUsername, const FText& ErrorMessage);

	enum class EResult : uint8
	{
		Retry,
		Skip
	};

	EResult GetResult() const
	{
		return Result;
	}

	const FString& GetPort() const
	{
		return Port;
	}

	const FString& GetUserName() const
	{
		return UserName;
	}

private:

	void CloseModalDialog();

	FReply OnResetToDefaults();
	FReply OnRetryConnection();
	FReply OnSkip();

	void OnUrlClicked() const;

	TWeakPtr<SWindow> WindowWidget;

	TSharedPtr<SEditableTextBox> PortTextWidget;
	TSharedPtr<SEditableTextBox> UsernameTextWidget;

	EResult Result = EResult::Skip;

	FString ConfigSectionName;

	FString Port;
	FString UserName;
};

} // namespace UE::Virtualization

#endif //UE_VA_WITH_SLATE
