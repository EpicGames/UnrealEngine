// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

#include "Features/IModularFeatures.h"
#include "VirtualizationSourceControlUtilities.h"

#if UE_VA_WITH_SLATE
#include "MessageLogModule.h"
#endif //UE_VA_WITH_SLATE

#define LOCTEXT_NAMESPACE "Virtualization"

namespace UE::Virtualization
{

class FVirtualizationModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		IModularFeatures::Get().RegisterModularFeature(FName("VirtualizationSourceControlUtilities"), &SourceControlutility);

#if UE_VA_WITH_SLATE
		FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
		MessageLogModule.RegisterLogListing("LogVirtualization", LOCTEXT("AssetVirtualizationLogLabel", "Asset Virtualization"));
#endif //UE_VA_WITH_SLATE
	}

	virtual void ShutdownModule() override
	{
#if UE_VA_WITH_SLATE
		if (FModuleManager::Get().IsModuleLoaded("MessageLog"))
		{
			FMessageLogModule& MessageLogModule = FModuleManager::GetModuleChecked<FMessageLogModule>("MessageLog");
			MessageLogModule.UnregisterLogListing("LogVirtualization&");
		}
#endif //UE_VA_WITH_SLATE	

		IModularFeatures::Get().UnregisterModularFeature(FName("VirtualizationSourceControlUtilities"), &SourceControlutility);
	}

private:
	Experimental::FVirtualizationSourceControlUtilities SourceControlutility;
};

} // namespace UE::Virtualization

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(UE::Virtualization::FVirtualizationModule, Virtualization);
