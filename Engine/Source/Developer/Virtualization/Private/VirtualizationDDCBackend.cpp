// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualizationDDCBackend.h"

#include "Misc/Parse.h"
#include "DerivedDataCache.h"
#include "DerivedDataCacheRecord.h"
#include "DerivedDataRequestOwner.h"
#include "DerivedDataValue.h"

namespace UE::Virtualization
{

/** Utility function to help convert from UE::Virtualization::FIoHash to UE::DerivedData::FValueId */
static UE::DerivedData::FValueId ToDerivedDataValueId(const FIoHash& Id)
{
	return UE::DerivedData::FValueId::FromHash(Id);
}

FDDCBackend::FDDCBackend(FStringView ProjectName, FStringView ConfigName, FStringView InDebugName)
	: IVirtualizationBackend(ConfigName, InDebugName, EOperations::Push | EOperations::Pull)
	, BucketName(TEXT("BulkData"))
	, TransferPolicy(UE::DerivedData::ECachePolicy::None)
	, QueryPolicy(UE::DerivedData::ECachePolicy::None)
{
	
}

bool FDDCBackend::Initialize(const FString& ConfigEntry)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Initialize::Initialize);
	
	FString BucketNameIniFile;
	if(FParse::Value(*ConfigEntry, TEXT("Bucket="), BucketNameIniFile))
	{
		BucketName = BucketNameIniFile;
	}

	bool bAllowLocal = true;
	FParse::Bool(*ConfigEntry, TEXT("LocalStorage="), bAllowLocal);
	
	bool bAllowRemote = true;
	FParse::Bool(*ConfigEntry, TEXT("RemoteStorage="), bAllowRemote);

	UE_LOGF(LogVirtualization, Log, "[%ls] Bucket set to '%ls", *GetDebugName(), *BucketName);
	UE_LOGF(LogVirtualization, Log, "[%ls] Use of local storage set to '%ls", *GetDebugName(), bAllowLocal ? TEXT("true") : TEXT("false"));
	UE_LOGF(LogVirtualization, Log, "[%ls] Use of remote storage set to '%ls", *GetDebugName(), bAllowRemote ? TEXT("true") : TEXT("false"));

	if (!bAllowLocal && !bAllowRemote)
	{
		UE_LOGF(LogVirtualization, Error, "[%ls] LocalStorage and RemoteStorage cannot both be disabled", *GetDebugName());
		return false;
	}

	if (bAllowLocal)
	{
		TransferPolicy |= UE::DerivedData::ECachePolicy::Local;
		QueryPolicy |= UE::DerivedData::ECachePolicy::QueryLocal;
	}

	if (bAllowRemote)
	{
		TransferPolicy |= UE::DerivedData::ECachePolicy::Remote;
		QueryPolicy |= UE::DerivedData::ECachePolicy::QueryRemote;
	}

	Bucket = UE::DerivedData::FCacheBucket(BucketName);

	return true;	
}

IVirtualizationBackend::EConnectionStatus FDDCBackend::OnConnect()
{
	return IVirtualizationBackend::EConnectionStatus::Connected;
}

bool FDDCBackend::PushData(TArrayView<FPushRequest> Requests, EPushFlags Flags)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDDCBackend::PushData);

	UE::DerivedData::ICache& Cache = UE::DerivedData::GetCache();

	UE::DerivedData::FRequestOwner Owner(UE::DerivedData::EPriority::Normal);

	bool bWasSuccess = true;

	const bool bEnableExistenceCheck = !EnumHasAllFlags(Flags, EPushFlags::Force);

	// TODO: We tend not to memory bloat too much on large batches as the requests complete quite quickly
	// however we might want to consider adding better control on how much total memory we can dedicate to
	// loading payloads before we wait for requests to complete?
	for (FPushRequest& Request : Requests)
	{
		if (bEnableExistenceCheck && DoesPayloadExist(Request.GetIdentifier()))
		{
			Request.SetResult(FPushResult::GetAsAlreadyExists());
		}
		else
		{
			UE::DerivedData::FRequestBarrier Barrier(Owner);

			UE::DerivedData::FCacheKey Key;
			Key.Bucket = Bucket;
			Key.Hash = Request.GetIdentifier();

			UE::DerivedData::FValue DerivedDataValue(Request.GetPayload());
			check(DerivedDataValue.GetRawHash() == Request.GetIdentifier());

			UE::DerivedData::FCacheRecordBuilder RecordBuilder(Key);
			RecordBuilder.AddValue(ToDerivedDataValueId(Request.GetIdentifier()), DerivedDataValue);

			auto Callback = [&Request, &bWasSuccess](UE::DerivedData::FCachePutResponse&& Response)
			{
				if (Response.Status == UE::DerivedData::EStatus::Ok)
				{
					Request.SetResult(FPushResult::GetAsPushed());
				}
				else
				{
					Request.SetResult(FPushResult::GetAsError());
					bWasSuccess = false;
				}
			};

			// TODO: Improve the name when we start passing more context to this function
			Cache.Put({ {{TEXT("Mirage")}, RecordBuilder.Build(), TransferPolicy} }, Owner, MoveTemp(Callback));
		}
	}

	Owner.Wait();

	return bWasSuccess;
}

bool FDDCBackend::PullData(TArrayView<FPullRequest> Requests, EPullFlags Flags, FText& OutErrors)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDDCBackend::PullData);

	UE::DerivedData::ICache& Cache = UE::DerivedData::GetCache();

	UE::DerivedData::FRequestOwner Owner(UE::DerivedData::EPriority::Normal);

	for (FPullRequest& Request : Requests)
	{
		UE::DerivedData::FRequestBarrier Barrier(Owner);

		UE::DerivedData::FCacheKey Key;
		Key.Bucket = Bucket;
		Key.Hash = Request.GetIdentifier();

		auto Callback = [&Request](UE::DerivedData::FCacheGetResponse&& Response)
		{
			if (Response.Status == UE::DerivedData::EStatus::Ok)
			{
				UE::DerivedData::FValueId ValueId = ToDerivedDataValueId(Request.GetIdentifier());
				Request.SetPayload(Response.Record.GetValue(ValueId).GetData());
			}
		};

		// TODO: Improve the name when we start passing more context to this function
		Cache.Get({ {{TEXT("Mirage")}, Key, TransferPolicy} }, Owner, MoveTemp(Callback));
	}

	Owner.Wait();

	return true;
}

bool FDDCBackend::DoesPayloadExist(const FIoHash& Id)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDDCBackend::DoesPayloadExist);

	UE::DerivedData::ICache& Cache = UE::DerivedData::GetCache();

	UE::DerivedData::FCacheKey Key;
	Key.Bucket = Bucket;
	Key.Hash = Id;

	UE::DerivedData::FRequestOwner Owner(UE::DerivedData::EPriority::Blocking);
	
	UE::DerivedData::EStatus ResultStatus;
	auto Callback = [&ResultStatus](UE::DerivedData::FCacheGetResponse&& Response)
	{
		ResultStatus = Response.Status;
	};

	// TODO: Improve the name when we start passing more context to this function
	Cache.Get({{{TEXT("Mirage")}, Key, QueryPolicy | UE::DerivedData::ECachePolicy::SkipData}}, Owner, MoveTemp(Callback));

	Owner.Wait();

	return ResultStatus == UE::DerivedData::EStatus::Ok;
}

UE_REGISTER_VIRTUALIZATION_BACKEND_FACTORY(FDDCBackend, DDCBackend);
} // namespace UE::Virtualization