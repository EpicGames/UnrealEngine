// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "IVirtualizationBackend.h"

#include "DerivedDataCacheKey.h"

namespace UE::DerivedData { enum class ECachePolicy : uint32; }

namespace UE::Virtualization
{
/**
 * A backend that uses the DDC2 as it's storage mechanism. It is intended to be used as a local caching
 * system to speed up operations rather than for use as persistent storage.
 * 
 * Ini file setup:
 * 'Name'=(Type=DDCBackend, Bucket="XXX", LocalStorage=True/False, RemoteStorage=True/False)
 * 
 * Required Values:
 * 'Name': The backend name in the hierarchy.
 * 
 * Optional Values:
 * Bucket: An alphanumeric identifier used to group the payloads together in storage. [Default="BulkData"]
 * LocalStorage: When set to true, the payloads can be stored locally. [Default=true]
 * RemoteStorage: When set to true, the payloads can be stored remotely. [Default=true]
 */
class FDDCBackend final : public IVirtualizationBackend
{
public:
	explicit FDDCBackend(FStringView ProjectName, FStringView ConfigName, FStringView InDebugName);
	virtual ~FDDCBackend() = default;

private:
	/* IVirtualizationBackend implementation */

	virtual bool Initialize(const FString& ConfigEntry) override;

	virtual EConnectionStatus OnConnect() override;

	virtual bool PushData(TArrayView<FPushRequest> Requests, EPushFlags Flags) override;
	virtual bool PullData(TArrayView<FPullRequest> Requests, EPullFlags Flags, FText& OutErrors) override;
	
	virtual bool DoesPayloadExist(const FIoHash& Id) override;

private:
	/** The bucket being used to group together the virtualized payloads in storage */
	FString BucketName;

	/** The FCacheBucket used with the DDC, cached to avoid recreating it for each request */
	UE::DerivedData::FCacheBucket Bucket;

	/** The policy to use when uploading or downloading data from the cache */
	UE::DerivedData::ECachePolicy TransferPolicy;
	/** The policy to use when querying the cache for information */
	UE::DerivedData::ECachePolicy QueryPolicy;
};

} // namespace UE::Virtualization