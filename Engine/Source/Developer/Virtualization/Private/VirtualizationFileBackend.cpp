// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualizationFileBackend.h"

#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "Misc/Parse.h"
#include "Misc/Paths.h"
#include "VirtualizationUtilities.h"

namespace UE::Virtualization
{

FFileSystemBackend::FFileSystemBackend(FStringView ProjectName, FStringView ConfigName, FStringView DebugName)
	: IVirtualizationBackend(ConfigName, DebugName, EOperations::Push | EOperations::Pull)
{
}

bool FFileSystemBackend::Initialize(const FString& ConfigEntry)
{
	if (!FParse::Value(*ConfigEntry, TEXT("Path="), RootDirectory))
	{
		RootDirectory = *WriteToString<512>(FPaths::ProjectSavedDir(), TEXT("VAPayloads"));
	}

	FString EnvOverrideName;
	if (FParse::Value(*ConfigEntry, TEXT("EnvPathOverride="), EnvOverrideName))
	{
		FString OverridePath = FPlatformMisc::GetEnvironmentVariable(*EnvOverrideName);
		if (!OverridePath.IsEmpty())
		{
			UE_LOGF(LogVirtualization, Log, "[%ls] Overriding path with envvar '%ls'", *GetDebugName(), *EnvOverrideName);
			RootDirectory = OverridePath;
		}
	}

	FPaths::NormalizeDirectoryName(RootDirectory);
	FPaths::RemoveDuplicateSlashes(RootDirectory);
	
	if (RootDirectory.IsEmpty())
	{
		UE_LOGF(LogVirtualization, Error, "[%ls] Config file entry 'Path=' was empty", *GetDebugName());
		return false;
	}

	// TODO: Validate that the given path is usable?

	int32 RetryCountIniFile = INDEX_NONE;
	if (FParse::Value(*ConfigEntry, TEXT("RetryCount="), RetryCountIniFile))
	{
		RetryCount = RetryCountIniFile;
	}

	int32 RetryWaitTimeMSIniFile = INDEX_NONE;
	if (FParse::Value(*ConfigEntry, TEXT("RetryWaitTime="), RetryWaitTimeMSIniFile))
	{
		RetryWaitTimeMS = RetryWaitTimeMSIniFile;
	}

	// Now log a summary of the backend settings to make issues easier to diagnose
	UE_LOGF(LogVirtualization, Log, "[%ls] Using path: '%ls'", *GetDebugName(), *FPaths::ConvertRelativePathToFull(RootDirectory));
	UE_LOGF(LogVirtualization, Log, "[%ls] Will retry failed read attempts %d times with a gap of %dms between them", *GetDebugName(), RetryCount, RetryWaitTimeMS);

	return true;
}

IVirtualizationBackend::EConnectionStatus FFileSystemBackend::OnConnect()
{
	return IVirtualizationBackend::EConnectionStatus::Connected;
}

bool FFileSystemBackend::PushData(TArrayView<FPushRequest> Requests, EPushFlags Flags)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FFileSystemBackend::PushData);

	int32 ErrorCount = 0;

	const bool bEnableExistenceCheck = !EnumHasAllFlags(Flags, EPushFlags::Force);

	for (FPushRequest& Request : Requests)
	{
		const FIoHash& PayloadId = Request.GetIdentifier();
	
		if (bEnableExistenceCheck && DoesPayloadExist(PayloadId))
		{
			UE_LOGF(LogVirtualization, Verbose, "[%ls] Already has a copy of the payload '%ls'.", *GetDebugName(), *LexToString(PayloadId));
			Request.SetResult(FPushResult::GetAsAlreadyExists());

			continue;
		}

		// Make sure to log any disk write failures to the user, even if this backend will often be optional as they are
		// not expected and could indicate bigger problems.
		// 
		// First we will write out the payload to a temp file, after which we will move it to the correct storage location
		// this helps reduce the chance of leaving corrupted data on disk in the case of a power failure etc.
		const FString TempFilePath = FPaths::CreateTempFilename(*FPaths::ProjectSavedDir(), TEXT("vapayload"));

		TUniquePtr<FArchive> FileAr(IFileManager::Get().CreateFileWriter(*TempFilePath));

		if (FileAr == nullptr)
		{
			TStringBuilder<MAX_SPRINTF> SystemErrorMsg;
			Utils::GetFormattedSystemError(SystemErrorMsg);

			UE_LOGF(LogVirtualization, Error, "[%ls] Failed to write payload '%ls' to '%ls' due to system error: %ls",
				*GetDebugName(),
				*LexToString(PayloadId),
				*TempFilePath,
				SystemErrorMsg.ToString());

			ErrorCount++;
			Request.SetResult(FPushResult::GetAsError());

			continue;
		}

		{
			FCompressedBuffer Payload = Request.GetPayload();
			*FileAr << Payload;
		}

		if (!FileAr->Close())
		{
			TStringBuilder<MAX_SPRINTF> SystemErrorMsg;
			Utils::GetFormattedSystemError(SystemErrorMsg);

			UE_LOGF(LogVirtualization, Error, "[%ls] Failed to write payload '%ls' contents to '%ls' due to system error: %ls",
				*GetDebugName(),
				*LexToString(PayloadId),
				*TempFilePath,
				SystemErrorMsg.ToString());

			IFileManager::Get().Delete(*TempFilePath, true, false, true);  // Clean up the temp file if it is still around but do not failure cases to the user

			ErrorCount++;
			Request.SetResult(FPushResult::GetAsError());

			continue;
		}

		TStringBuilder<512> FilePath;
		CreateFilePath(PayloadId, FilePath);

		// If the file already exists we don't need to replace it, we will also do our own error logging.
		if (IFileManager::Get().Move(FilePath.ToString(), *TempFilePath, /*Replace*/ false, /*EvenIfReadOnly*/ false, /*Attributes*/ false, /*bDoNotRetryOrError*/ true))
		{
			Request.SetResult(FPushResult::GetAsPushed());
		}
		else
		{
			// Store the error message in case we need to display it
			TStringBuilder<MAX_SPRINTF> SystemErrorMsg;
			Utils::GetFormattedSystemError(SystemErrorMsg);

			IFileManager::Get().Delete(*TempFilePath, true, false, true); // Clean up the temp file if it is still around but do not failure cases to the user

			// Check if another thread or process was writing out the payload at the same time, if so we 
			// don't need to give an error message.
			if (DoesPayloadExist(PayloadId))
			{
				UE_LOGF(LogVirtualization, Verbose, "[%ls] Already has a copy of the payload '%ls'.", *GetDebugName(), *LexToString(PayloadId));
				Request.SetResult(FPushResult::GetAsAlreadyExists());
				continue;
			}
			else
			{
				UE_LOGF(LogVirtualization, Error, "[%ls] Failed to move payload '%ls' to it's final location '%ls' due to system error: %ls",
					*GetDebugName(),
					*LexToString(PayloadId),
					*FilePath,
					SystemErrorMsg.ToString());

				ErrorCount++;
				Request.SetResult(FPushResult::GetAsError());
			}
		}
	}

	return ErrorCount == 0;
}

bool FFileSystemBackend::PullData(TArrayView<FPullRequest> Requests, EPullFlags Flags, FText& OutErrors)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FFileSystemBackend::PullData);

	for (FPullRequest& Request : Requests)
	{
		TStringBuilder<512> FilePath;
		CreateFilePath(Request.GetIdentifier(), FilePath);

		// TODO: Should we allow the error severity to be configured via ini or just not report this case at all?
		if (!IFileManager::Get().FileExists(FilePath.ToString()))
		{
			UE_LOGF(LogVirtualization, VeryVerbose, "[%ls] Does not contain the payload '%ls'", 
				*GetDebugName(), 
				*LexToString(Request.GetIdentifier()));

			continue;
		}

		TUniquePtr<FArchive> FileAr = OpenFileForReading(FilePath.ToString());

		if (FileAr == nullptr)
		{
			TStringBuilder<MAX_SPRINTF> SystemErrorMsg;
			Utils::GetFormattedSystemError(SystemErrorMsg);

			UE_LOGF(LogVirtualization, Error, "[%ls] Failed to load payload '%ls' from file '%ls' due to system error: %ls",
				*GetDebugName(),
				*LexToString(Request.GetIdentifier()),
				FilePath.ToString(),
				SystemErrorMsg.ToString());

			Request.SetError();

			continue;
		}

		Request.SetPayload(FCompressedBuffer::Load(*FileAr));

		if (FileAr->IsError())
		{
			UE_LOGF(LogVirtualization, Error, "[%ls] Failed to serialize payload '%ls' from file '%ls'",
				*GetDebugName(),
				*LexToString(Request.GetIdentifier()),
				FilePath.ToString());

			Request.SetError();

			continue;
		}
	}

	return true;
}

bool FFileSystemBackend::DoesPayloadExist(const FIoHash& Id)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FFileSystemBackend::DoesPayloadExist);

	TStringBuilder<512> FilePath;
	CreateFilePath(Id, FilePath);

	return IFileManager::Get().FileExists(FilePath.ToString());
}

void FFileSystemBackend::CreateFilePath(const FIoHash& PayloadId, FStringBuilderBase& OutPath)
{
	TStringBuilder<52> PayloadPath;
	Utils::PayloadIdToPath(PayloadId, PayloadPath);

	OutPath << RootDirectory << TEXT("/") << PayloadPath;
}

TUniquePtr<FArchive> FFileSystemBackend::OpenFileForReading(const TCHAR* FilePath)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FFileSystemBackend::OpenFileForReading);

	int32 Retries = 0;

	while (Retries < RetryCount)
	{
		TUniquePtr<FArchive> FileAr(IFileManager::Get().CreateFileReader(FilePath));
		if (FileAr)
		{
			return FileAr;
		}
		else
		{
			UE_LOGF(LogVirtualization, Warning, "[%ls] Failed to open '%ls' for reading attempt retrying (%d/%d) in %dms...", *GetDebugName(), FilePath, Retries, RetryCount, RetryWaitTimeMS);
			FPlatformProcess::SleepNoStats(static_cast<float>(RetryWaitTimeMS) * 0.001f);

			Retries++;
		}
	}

	return nullptr;
}

UE_REGISTER_VIRTUALIZATION_BACKEND_FACTORY(FFileSystemBackend, FileSystem);

} // namespace UE::Virtualization
