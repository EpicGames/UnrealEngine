// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IVirtualizationBackend.h"

namespace UE::Virtualization
{
/**
 * A basic backend based on the file system. This can be used to access/store virtualization
 * data either on a local disk or a network share. It is intended to be used as a caching system
 * to speed up operations (running a local cache or a shared cache for a site) rather than as the
 * proper backend solution.
 *
 * Ini file setup:
 * 'Name'=(Type=FileSystem, Path="XXX", RetryCount=X, RetryWaitTime=X)
 * 
 * Required Values:
 * 'Name': The backend name in the hierarchy.
 * 'Type': The backend will be of type 'FFileSystemBackend'.
 * 'Path': The root directory where the files are stored.
 *
 * Optional Values:
 * RetryCount:		How many times we should try to open a payload file for read before giving up with
 *					an error. Useful when many threads/processes can be pushing/pulling from the same
 *					path/ [Default=10]
 * RetryWaitTime:	The length of time the process should wait between each read attempt in milliseconds.
 *					Remember that the max length of time that the process can stall attempting to read
 *					a payload file is RetryCount * RetryWaitTime; [Default=100ms]
 * EnvPathOverride: The name of the environment variable that the user can set to override 'Path'. This 
 *					should only ever be set if you wish for the users to be able to choose where the 
 *					payloads are being stored and do not rely on 'Path' always being used. [Default=""]
 */
class FFileSystemBackend final : public IVirtualizationBackend
{
public:
	explicit FFileSystemBackend(FStringView ProjectName, FStringView ConfigName, FStringView DebugName);
	virtual ~FFileSystemBackend() = default;

private:
	/* IVirtualizationBackend implementation */

	virtual bool Initialize(const FString& ConfigEntry) override;

	virtual EConnectionStatus OnConnect() override;

	virtual bool PushData(TArrayView<FPushRequest> Requests, EPushFlags Flags) override;
	virtual bool PullData(TArrayView<FPullRequest> Requests, EPullFlags Flags, FText& OutErrors) override;

	virtual bool DoesPayloadExist(const FIoHash& Id) override;
	
private:

	void CreateFilePath(const FIoHash& PayloadId, FStringBuilderBase& OutPath);

	TUniquePtr<FArchive> OpenFileForReading(const TCHAR* FilePath);

	/** The root directory where the payload files should be located. */
	FString RootDirectory;

	/** The number of times to retry opening a payload file for read */
	int32 RetryCount = 10;
	/** The length of time (in milliseconds) to wait after each attempt before retrying. */
	int32 RetryWaitTimeMS = 100;
};

} // namespace UE::Virtualization