// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualizationFilterSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VirtualizationFilterSettings)

UVirtualizationFilterSettings::UVirtualizationFilterSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

