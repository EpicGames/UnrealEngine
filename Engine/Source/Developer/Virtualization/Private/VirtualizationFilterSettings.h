// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "VirtualizationFilterSettings.generated.h"

/** 
 * Settings to use when determining if a package should be included/excluded from the
 * content virtualization process.
 */ 
UCLASS(config = Engine)
class UVirtualizationFilterSettings : public UObject
{
	GENERATED_UCLASS_BODY()

	/**
	 * A list of package paths that we should always prevent being virtualized .
	 * Each path can either be a full package path to a specific package or it can be a
	 * path to a directory, in which case all packages under that directory (and subdirectories)
	 * will be excluded from the virtualization process.
	 */
	UPROPERTY(Config)
	TArray<FString> ExcludePackagePaths;

	/**
	 * A list of package paths that we should always attempt to virtualize.
	 * Each path can either be a full package path to a specific package or it can be a
	 * path to a directory, in which case all packages under that directory (and subdirectories)
	 * will be included in the virtualization process.
	 */
	UPROPERTY(Config)
	TArray<FString> IncludePackagePaths;
};
