// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualizationManager.h"

#include "AnalyticsEventAttribute.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformTime.h"
#include "Misc/App.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/MessageDialog.h"
#include "Misc/PackageName.h"
#include "Misc/PackagePath.h"
#include "Misc/Parse.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "Misc/ScopedSlowTask.h"
#include "PackageRehydrationProcess.h"
#include "PackageVirtualizationProcess.h"
#include "ProfilingDebugging/CookStats.h"
#include "VirtualizationFilterSettings.h"
#include "VirtualizationUtilities.h"

#define LOCTEXT_NAMESPACE "Virtualization"

namespace UE::Virtualization
{
UE_REGISTER_VIRTUALIZATION_SYSTEM(UE::Virtualization::FVirtualizationManager, Default);

ENUM_CLASS_FLAGS(FVirtualizationManager::ECachingPolicy);

// Can be defined as 1 by programs target.cs files force the backend connections
// to lazy initialize on first use rather than when the system is initialized.
#ifndef UE_VIRTUALIZATION_CONNECTION_LAZY_INIT
	#define UE_VIRTUALIZATION_CONNECTION_LAZY_INIT 0
#endif //UE_VIRTUALIZATION_CONNECTION_LAZY_INIT

#define UE_INLINE_ALLOCATION_COUNT 4

// TODO: Move to RegisterConsoleCommands
static TAutoConsoleVariable<bool> CVarLazyInitConnections(
	TEXT("VA.LazyInitConnections"),
	false,
	TEXT("When true the VA backends will defer creating their connections until first use"));

//
#define UE_USE_GLOBAL_CVAR 1

#if UE_USE_GLOBAL_CVAR

static FAutoConsoleVariable CVarAllowPkgVirtualization(
	TEXT("VA.AllowPkgVirtualization"),
	true,
	TEXT("When true submitting packages in the editor will no longer trigger the virtualization process")
);

#endif // UE_USE_GLOBAL_CVAR

/** Utility struct, similar to FScopeLock but allows the lock to be enabled/disabled more easily */
struct FConditionalScopeLock
{
	UE_NONCOPYABLE(FConditionalScopeLock);

	FConditionalScopeLock(FCriticalSection* InSyncObject, bool bShouldLock)
	{
		checkf(InSyncObject != nullptr, TEXT("InSyncObject must point to a valid FCriticalSection"));

		if (bShouldLock)
		{		
			SyncObject = InSyncObject;
			SyncObject->Lock();
		}
		else
		{
			SyncObject = nullptr;
		}
	}

	/** Destructor that performs a release on the synchronization object. */
	~FConditionalScopeLock()
	{
		if (SyncObject != nullptr)
		{
			SyncObject->Unlock();
		}
	}

private:
	FCriticalSection* SyncObject;
};

/**
 * Utility to disable 'GIsRunningUnattendedScript' for a given scope.
 * We only change 'GIsRunningUnattendedScript' if called on the GameThread as the
 * value is not thread safe and is generally only changed on that thread.
 * 
 * We need this so that we can try to force a modal dialog to be shown during
 * the editor if a payload fails to pull as our only other option is to
 * terminate the process.
 */
struct FDisableUnattendedScriptGlobal
{
	FDisableUnattendedScriptGlobal()
	{
		if (IsInGameThread())
		{
			bOriginalValue = GIsRunningUnattendedScript;
			GIsRunningUnattendedScript = false;
		}
	}

	~FDisableUnattendedScriptGlobal()
	{
		if (IsInGameThread())
		{
			GIsRunningUnattendedScript = bOriginalValue;
		}
	}

private:
	bool bOriginalValue = false;
};

/** 
 * Utility class to help manage pull requests. When created it will remove invalid and duplicate requests so
 * that backends do not need to worry about them.
 * It will also prune the list of requests after each backend pull so that we only make requests for payloads 
 * that have not yet been found and on request provide a list of payloads that should be cached.
 * On destruction it will then write the results back to the original requests. Duplicate requests will share 
 * references to the same payload in memory. Once destroyed all of the original requests should have a success
 * or error status and none should be listed as pending.
 * 
 * NOTE: This is intended to be used by FVirtualizationManager::PullDataFromAllBackends only, hence doing a 
 * few dangerous things like returning a TArrayView as we know it will not be misused. If this is ever taken
 * into wider use we will have to revisit things like that.
 */
class FPullRequestCollection
{
public:
	UE_NONCOPYABLE(FPullRequestCollection);

	FPullRequestCollection() = delete;
	FPullRequestCollection(TArrayView<FPullRequest> InRequests)
		: OriginalRequests(InRequests)
	{
		CurrentRequests.Reserve(OriginalRequests.Num());

		// Record each payload hash as we add it to CurrentRequests so that we only request duplicates once.
		TSet<FIoHash, DefaultKeyFuncs<FIoHash>, TInlineSetAllocator<UE_INLINE_ALLOCATION_COUNT>> UniquePayloads;

		for (FPullRequest& Request : OriginalRequests)
		{
			if (Request.GetIdentifier().IsZero())
			{
				Request.SetError();
				UE_LOGF(LogVirtualization, Error, "Attempting to pull a virtualized payload with an invalid FIoHash");
			}
			else if (!UniquePayloads.Contains(Request.GetIdentifier()))
			{
				CurrentRequests.Add(Request);
				UniquePayloads.Add(Request.GetIdentifier());
			}
		}
	}

	~FPullRequestCollection()
	{
		for (FPullRequest& Request : OriginalRequests)
		{
			if (FCompressedBuffer* Payload = LoadedPayloads.Find(Request.GetIdentifier()))
			{
				Request.SetPayload(*Payload);
			}
			else
			{
				Request.SetError();
			}
		}
	}

	/** 
	 * Called after the requests from ::GetRequests have been pulled from a backend. Payloads that were
	 * successfully pulled will be removed from the request list and added to the LoadedPayloads map 
	 * so that they can be assigned to the original requests later.
	 * 
	 * @param Backend					The backend that the payloads were pulled from
	 * @param bRequirePayloadsToCache	Do we need a list of payloads that should be cached
	 * 
	 * @return A list of requests that now need to be cached (if required)
	 */
	TArray<FPushRequest> OnPullCompleted(const IVirtualizationBackend& Backend, bool bRequirePayloadsToCache)
	{
		TArray<FPushRequest> PayloadsToCache;
		if (bRequirePayloadsToCache)
		{
			PayloadsToCache.Reserve(CurrentRequests.Num());
		}

		for (int32 Index = 0; Index < CurrentRequests.Num();)
		{
			const FPullRequest& Request = CurrentRequests[Index];
			if (Request.IsSuccess())
			{
				if (bRequirePayloadsToCache)
				{
					PayloadsToCache.Emplace(FPushRequest(Request.GetIdentifier(), Request.GetPayload(), FString()));
				}

				LoadedPayloads.Add(Request.GetIdentifier(), Request.GetPayload());

				CurrentRequests.RemoveAtSwap(Index);

				UE_LOGF(LogVirtualization, VeryVerbose, "[%ls] pulled payload '%ls'", *Backend.GetDebugName(), *LexToString(Request.GetIdentifier()));
			}
			else
			{
				++Index;
			}
		}

		return PayloadsToCache;
	}

	/** Return the current list of requests that need to be made */
	TArrayView<FPullRequest> GetRequests()
	{
		return CurrentRequests;
	}

	TArray<FIoHash> GetFailedPayloads() const
	{
		TArray<FIoHash> FailedPayloads;

		for (const FPullRequest& Request : CurrentRequests)
		{
			if (!Request.IsSuccess())
			{
				FailedPayloads.Add(Request.GetIdentifier());
			}
		}

		return FailedPayloads;
	}

	/** Returns if there are still requests that need servicing or not */
	bool IsWorkComplete() const
	{
		return CurrentRequests.IsEmpty();
	}

private:
	TArrayView<FPullRequest> OriginalRequests;

	TMap<FIoHash, FCompressedBuffer, TInlineSetAllocator<UE_INLINE_ALLOCATION_COUNT>> LoadedPayloads;
	TArray<FPullRequest, TInlineAllocator<UE_INLINE_ALLOCATION_COUNT>> CurrentRequests;
};

const TCHAR* LexToString(EPackageFilterMode Value)
{
	switch (Value)
	{
	case EPackageFilterMode::OptIn:
		return TEXT("OptIn");
	case EPackageFilterMode::OptOut:
		return TEXT("OptOut");
	default:
		checkNoEntry();
		return TEXT("");
	}
}

bool LexTryParseString(EPackageFilterMode& OutValue, FStringView Buffer)
{
	if (Buffer == TEXT("OptOut"))
	{
		OutValue = EPackageFilterMode::OptOut;
		return true;
	}
	else if (Buffer == TEXT("OptIn"))
	{
		OutValue = EPackageFilterMode::OptIn;
		return true;
	}

	return false;
}

/**
 * Utility to check if a cmdline switch is present in the commandline under either of two names.
 * Useful when transitioning from one command line to another.
 */
static bool IsCmdLineParamSet(const TCHAR* Cmd, const TCHAR* AlternativeCmd)
{
	const TCHAR* CmdLine = FCommandLine::Get();

	if (FParse::Param(CmdLine, Cmd))
	{
		return true;
	}

	if (FParse::Param(CmdLine, AlternativeCmd))
	{
		return true;
	}

	return false;
}

/**
 * Utility to check if a cmdline value is present in the commandline under either of two names.
 * Useful when transitioning from one command line to another.
 */
template<typename T>
static bool IsCmdLineValueSet(const TCHAR* Cmd, const TCHAR* AlternativeCmd, T& OutValue)
{
	const TCHAR* CmdLine = FCommandLine::Get();

	if (FParse::Value(CmdLine, Cmd, OutValue))
	{
		return true;
	}

	if (FParse::Value(CmdLine, AlternativeCmd, OutValue))
	{
		return true;
	}

	return false;
}

/** Utility to set the same FPushResult on many requests at once */
static void SetPushRequestsResult(TArrayView<FPushRequest> Requests, FPushResult Result)
{
	for (FPushRequest& Request : Requests)
	{
		Request.SetResult(Result);
	}
}

/* Utility function for building up a lookup table of all available IBackendFactory interfaces*/
FVirtualizationManager::FRegistedFactories FindBackendFactories()
{
	FVirtualizationManager::FRegistedFactories BackendFactories;

	TArray<IVirtualizationBackendFactory*> FactoriesArray = IModularFeatures::Get().GetModularFeatureImplementations<IVirtualizationBackendFactory>(FName("VirtualizationBackendFactory"));
	for (IVirtualizationBackendFactory* FactoryInterface : FactoriesArray)
	{
		checkf(FactoryInterface != nullptr, TEXT("A nullptr was added to the modular features for 'VirtualizationBackendFactory'"));

		const FName FactoryName = FactoryInterface->GetName();

		if (!BackendFactories.Contains(FactoryName))
		{
			BackendFactories.Add(FactoryName, FactoryInterface);
		}
		else
		{
			UE_LOGF(LogVirtualization, Error, "Duplicate IBackendFactory found! Name '%ls'", *FactoryName.ToString());
		}
	}

	return BackendFactories;
}

/* Utility function for finding entries in a given string*/
TArray<FString> ParseEntries(const FString& Data)
{
	TArray<FString> Entries;

	const TCHAR* DataPtr = *Data;

	const TCHAR* EntryLabel = TEXT("Entry=");
	const int32 EntryLabelLength = FCString::Strlen(EntryLabel);

	FString ConfigEntryName;
	while (FParse::Value(DataPtr, EntryLabel, ConfigEntryName))
	{
		Entries.Add(ConfigEntryName);

		// Skip head so we can look for any additional entries (note that we might not skip past the existing
		// entry has we have no idea how much whitespace was ignored by FParse, but it will be enough)
		DataPtr += EntryLabelLength + ConfigEntryName.Len();
	}

	return Entries;
}

/** 
 * FPackagePath has strict requirements on how file paths are formated in order for it to be
 * able to accept them. In theory the paths that the virtualization system are given are
 * already in a good shape but as we do not know where they come from we cannot trust them.
 * This utility will iterate over a list of untrusted filepaths and call various FPaths methods
 * on them to make sure that they are all correct.
 */
TArray<FString> SanitizeFilePaths(TConstArrayView<FString> FilePaths)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SanitizeFilePaths);

	TArray<FString> SanitizedPaths;
	SanitizedPaths.Reserve(FilePaths.Num());

	for (const FString& Path : FilePaths)
	{
		FString SanitizedPath = Path;

		FPaths::NormalizeFilename(SanitizedPath);
		FPaths::RemoveDuplicateSlashes(SanitizedPath);

		SanitizedPaths.Emplace(MoveTemp(SanitizedPath));
	}

	return SanitizedPaths;
}

/**
 * Profiling data allowing us to track how payloads are being push/pulled during the lifespan of the process. Note that as all backends are
 * created at the same time, we don't need to add locked when accessing the maps. In addition FCookStats is thread safe when adding hits/misses
 * so we don't have to worry about that either.
 * We keep the FCookStats here rather than as a member of IVirtualizationBackend to try and avoid the backends needing to be aware of the data that
 * we are gathering at all. This way all profiling code is kept to this cpp.
 */
namespace Profiling
{
#if ENABLE_COOK_STATS
	TMap<FString, FCookStats::CallStats> CacheStats;
	TMap<FString, FCookStats::CallStats> PushStats;
	TMap<FString, FCookStats::CallStats> PullStats;

	void CreateStats(const IVirtualizationBackend& Backend)
	{
		CacheStats.Add(Backend.GetDebugName());
		PushStats.Add(Backend.GetDebugName());
		PullStats.Add(Backend.GetDebugName());
	}

	FCookStats::CallStats& GetCacheStats(const IVirtualizationBackend& Backend)
	{
		return *CacheStats.Find(Backend.GetDebugName());
	}
	
	FCookStats::CallStats& GetPushStats(const IVirtualizationBackend& Backend)
	{
		return *PushStats.Find(Backend.GetDebugName());
	}

	FCookStats::CallStats& GetPullStats(const IVirtualizationBackend& Backend)
	{
		return *PullStats.Find(Backend.GetDebugName());
	}

	bool HasProfilingData(const TMap<FString, FCookStats::CallStats>& Stats)
	{
		for (const auto& Iterator : Stats)
		{
			if (Iterator.Value.GetAccumulatedValueAnyThread(FCookStats::CallStats::EHitOrMiss::Hit, FCookStats::CallStats::EStatType::Counter) > 0)
			{
				return true;
			}

			if (Iterator.Value.GetAccumulatedValueAnyThread(FCookStats::CallStats::EHitOrMiss::Miss, FCookStats::CallStats::EStatType::Counter) > 0)
			{
				return true;
			}
		}

		return false;
	}

	/** Returns true if we have gathered any profiling data at all */
	bool HasProfilingData()
	{
		return HasProfilingData(CacheStats) || HasProfilingData(PushStats) || HasProfilingData(PullStats);
	}

	void LogStats()
	{
		if (!HasProfilingData())
		{
			UE_LOGF(LogVirtualization, Display, "Skipping profile data as there was no activity to report");
			return; // Early out if we have no data
		}

		UE_LOGF(LogVirtualization, Display, "");
		UE_LOGF(LogVirtualization, Display, "Virtualization ProfileData");
		UE_LOGF(LogVirtualization, Display, "=============================================================================================");

		auto PrintStats = [](const TCHAR* Name, const TMap<FString, FCookStats::CallStats>& Stats)
			{
				if (HasProfilingData(Stats))
				{
					UE_LOGF(LogVirtualization, Display, "%-40ls|%10ls|%15ls|%12ls|%11ls|", Name, TEXT("TotalCount"), TEXT("TotalSize (MiB)"), TEXT("TotalTime(s)"), TEXT("AvgTime(ms)"));
					UE_LOGF(LogVirtualization, Display, "----------------------------------------|----------|---------------|------------|-----------|");

					for (const auto& Iterator : Stats)
					{
						const int64 Count = Iterator.Value.GetAccumulatedValueAnyThread(FCookStats::CallStats::EHitOrMiss::Hit, FCookStats::CallStats::EStatType::Counter);
						const double TotalTime = (double)Iterator.Value.GetAccumulatedValueAnyThread(FCookStats::CallStats::EHitOrMiss::Hit, FCookStats::CallStats::EStatType::Cycles) * FPlatformTime::GetSecondsPerCycle();
						const double DataSizeMB = (double)Iterator.Value.GetAccumulatedValueAnyThread(FCookStats::CallStats::EHitOrMiss::Hit, FCookStats::CallStats::EStatType::Bytes) / (1024.0f * 1024.0f);
						const double AvgTime = Count > 0 ? (TotalTime * 1000.0) / static_cast<double>(Count) : 0.0;

						UE_LOGF(LogVirtualization, Display, "%-40.40ls|%10lld|%15.1lf|%12.1lf|%11.0lf|",
							*Iterator.Key,
							Count,
							DataSizeMB,
							TotalTime,
							AvgTime);
					}

					UE_LOGF(LogVirtualization, Display, "=============================================================================================");
				}
			};

		PrintStats(TEXT("Caching Data"), CacheStats);
		PrintStats(TEXT("Pushing Data"), PushStats);
		PrintStats(TEXT("Pulling Data"), PullStats);
	}
#endif // ENABLE_COOK_STATS
} //namespace Profiling

FString FVirtualizationManager::ConnectionHelpUrl;

FVirtualizationManager::FVirtualizationManager()
	: bAllowPackageVirtualization(true)
	, CachingPolicy(ECachingPolicy::AlwaysCache)
	, MinPayloadLength(0)
	, BackendGraphName(TEXT("ContentVirtualizationBackendGraph_None"))
	, VirtualizationProcessTag(TEXT("#virtualized"))
	, FilteringMode(EPackageFilterMode::OptOut)
	, bFilterMapContent(true)
	, bAllowSubmitIfVirtualizationFailed(false)
	, bLazyInitConnections(false)
	, bUseLegacyErrorHandling(false)
	, bForceCachingOnPull(false)
	, bAllowBackendReconnection(true)
{
}

FVirtualizationManager::~FVirtualizationManager()
{
	for (const TPair<IConsoleVariable*, FDelegateHandle>& KV : DebugValues.ConsoleDelegateHandles)
	{
		IConsoleVariable* ConsoleVariable = KV.Key;
		const FDelegateHandle& Handle = KV.Value;

		ConsoleVariable->OnChangedDelegate().Remove(Handle);
	}

	for (IConsoleObject* ConsoleObject : DebugValues.ConsoleObjects)
	{
		IConsoleManager::Get().UnregisterConsoleObject(ConsoleObject);
	}

	UE_LOGF(LogVirtualization, Verbose, "Destroying backends");

	CacheStorageBackends.Empty();
	PersistentStorageBackends.Empty();
	PullEnabledBackends.Empty();

	AllBackends.Empty(); // This will delete all backends and beyond this point all references to them are invalid

	UE_LOGF(LogVirtualization, Verbose, "Virtualization manager destroyed");
}

bool FVirtualizationManager::Initialize(const FInitParams& InitParams)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualizationManager::Initialize);

	UE_LOGF(LogVirtualization, Display, "Initializing the virtualization manager...");

	// TODO: Ideally we'd break this down further, or at least have a FScopedSlowTask for each
	// backend initialization but the slow task system will only update the UI every 0.2 seconds
	// so if we have too many small tasks we might show misleading data to the user, so it is 
	// better for us to have a single scope here at the top level and rely on UnrealInsights for
	// detailed profiling unless we do something to how FScopedSlowTask updates the UI.
	FScopedSlowTask SlowTask(1.0f, LOCTEXT("VAInitialize", "Initializing virtualized asset system..."));
	SlowTask.EnterProgressFrame(1.0f);

	ProjectName = InitParams.ProjectName;

	ApplySettingsFromConfigFiles(InitParams.ConfigFile);
	ApplySettingsFromFromCmdline();
	ApplySettingsFromCVar();

	ApplyDebugSettingsFromFromCmdline();

	// Do this after all of the command line settings have been processed and any 
	// requested debug value changes already set.
	RegisterConsoleCommands();

	MountBackends(InitParams.ConfigFile);

	UE_LOGF(LogVirtualization, Display, "Virtualization manager initialization completed");

	return true;
}

bool FVirtualizationManager::IsEnabled() const
{
	return !AllBackends.IsEmpty();
}

bool FVirtualizationManager::IsPushingEnabled(EStorageType StorageType) const
{
	if (!bAllowPackageVirtualization)
	{
		return false;
	}

	switch (StorageType)
	{
		case EStorageType::Cache:
			return !CacheStorageBackends.IsEmpty() && EnumHasAllFlags(CachingPolicy, ECachingPolicy::CacheOnPush);

		case EStorageType::Persistent:
			return !PersistentStorageBackends.IsEmpty();

		default:
			checkNoEntry();
			return false;
	}
}

EPayloadFilterReason FVirtualizationManager::FilterPayload(const UObject* Owner) const
{
	UE::Virtualization::EPayloadFilterReason PayloadFilter = UE::Virtualization::EPayloadFilterReason::None;
	if (!ShouldVirtualizeAsset(Owner))
	{
		PayloadFilter |= UE::Virtualization::EPayloadFilterReason::Asset;
	}

	// TODO: If we keep this feature long term then we might want to work this out in SavePackage.cpp and pass the info
	// via FLinkerSave rather than the following code.
	if (bFilterMapContent)
	{
		if (const UObject* Outer = Owner->GetOutermostObject())
		{
			if (const UClass* OuterClass = Outer->GetClass())
			{
				const FName OuterClassName = OuterClass->GetFName();
				if (OuterClassName == FName("Level") ||
					OuterClassName == FName("World") ||
					OuterClassName == FName("MapBuildDataRegistry"))
				{
					PayloadFilter |= UE::Virtualization::EPayloadFilterReason::MapContent;
				}
			}
		}
	}

	return PayloadFilter;
}

bool FVirtualizationManager::AllowSubmitIfVirtualizationFailed() const
{
	return bAllowSubmitIfVirtualizationFailed;
}

bool FVirtualizationManager::PushData(TArrayView<FPushRequest> Requests, EStorageType StorageType)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualizationManager::PushData);

	// No requests always counts as success
	if (Requests.IsEmpty())
	{
		return true;
	}

	// Attempting to virtualize when the system or process is disabled always counts as a failure
	if (!IsEnabled() || !bAllowPackageVirtualization)
	{
		SetPushRequestsResult(Requests, FPushResult::GetAsProcessDisabled());
		return false;
	}

	FBackendArray& Backends = StorageType == EStorageType::Cache ? CacheStorageBackends : PersistentStorageBackends;

	if (Backends.IsEmpty())
	{
		SetPushRequestsResult(Requests, FPushResult::GetAsNoBackend());
		return false;
	}

	TArray<FPushRequest> ValidatedRequests;
	ValidatedRequests.Reserve(Requests.Num());

	TArray<int32> OriginalToValidatedRequest; // Builds a mapping between Requests and ValidatedRequests for later
	OriginalToValidatedRequest.SetNum(Requests.Num());

	// Create a new list of FPushRequest, excluding the requests that should not be processed for what ever reason.
	for (int32 Index = 0; Index < Requests.Num(); ++Index)
	{
		OriginalToValidatedRequest[Index] = INDEX_NONE;

		FPushRequest& Request = Requests[Index];
		if (Request.GetIdentifier().IsZero() || Request.GetPayloadSize() == 0)
		{
			Request.SetResult(FPushResult::GetAsInvalid());
			continue;
		}

		if ((int64)Request.GetPayloadSize() < MinPayloadLength)
		{
			UE_LOG(	LogVirtualization, Verbose, TEXT("Pushing payload (id: %s) with context ('%s') was prevented as it is smaller (%" UINT64_FMT ") than the MinPayloadLength (%" INT64_FMT ")"),
					*LexToString(Request.GetIdentifier()),
					*Request.GetContext(),
					Request.GetPayloadSize(),
					MinPayloadLength);

			Request.SetResult(FPushResult::GetAsFiltered(EPayloadFilterReason::MinSize));
			continue;
		}

		if (!ShouldVirtualize(Request.GetContext()))
		{
			UE_LOGF(	LogVirtualization, Verbose, "Pushing payload (id: %ls) with context ('%ls') was prevented by filtering",
					*LexToString(Request.GetIdentifier()), 
					*Request.GetContext());
			
			Request.SetResult(FPushResult::GetAsFiltered(EPayloadFilterReason::Path));
			continue;
		}

		OriginalToValidatedRequest[Index] = ValidatedRequests.Num();
		ValidatedRequests.Add(Request);
	}

	// Early out if none of the requests require pushing after validation
	if (ValidatedRequests.IsEmpty())
	{
		return true;
	}

	FConditionalScopeLock _(&DebugValues.ForceSingleThreadedCS, DebugValues.bSingleThreaded);

	// TODO: Note that all push operations are currently synchronous, probably 
	// should change to async at some point, although this makes handling failed
	// pushed much more difficult.

	int32 ErrorCount = 0;
	bool bWasPayloadPushed = false;
	
	for (IVirtualizationBackend* Backend : Backends)
	{
		if (Backend->TryConnect(bAllowBackendReconnection) != IVirtualizationBackend::EConnectionStatus::Connected)
		{
			UE_LOGF(LogVirtualization, Verbose, "Cannot push to backend  '%ls' as it was unable to connect", *Backend->GetDebugName());
			continue;
		}

		const bool bResult = TryPushDataToBackend(*Backend, ValidatedRequests);

		UE_CLOGF(bResult == true, LogVirtualization, Verbose, "[%ls] Pushed '%d' payload(s)", *Backend->GetDebugName(), ValidatedRequests.Num());
		UE_CLOGF(bResult == false, LogVirtualization, Error, "[%ls] Failed to push '%d' payload(s)", *Backend->GetDebugName(), ValidatedRequests.Num());

		if (!bResult)
		{
			ErrorCount++;
		}
		
		// Debug operation to validate that the payload we just pushed can be retrieved from storage
		if (DebugValues.bValidateAfterPush && bResult == true && Backend->IsOperationSupported(IVirtualizationBackend::EOperations::Pull))
		{
			for (FPushRequest& PushRequest : ValidatedRequests)
			{
				FText Errors;
				FPullRequest PullRequest(PushRequest.GetIdentifier());
				PullDataFromBackend(*Backend, MakeArrayView(&PullRequest, 1), Errors);

				checkf(PushRequest.GetIdentifier() == PullRequest.GetPayload().GetRawHash(),
						TEXT("[%s] Failed to pull payload '%s' after it was pushed to backend"),
						*Backend->GetDebugName(),
						*LexToString(PushRequest.GetIdentifier()));
			}
		}
	}

	UE_CLOGF(ErrorCount == Backends.Num(), LogVirtualization, Error, "Failed to push '%d' payload(s) to any backend", ValidatedRequests.Num());

	// Now we need to update the statuses of the original list of requests with those from our validated list
	for (int32 Index = 0; Index < Requests.Num(); ++Index)
	{
		const int32 MappingIndex = OriginalToValidatedRequest[Index];
		if (MappingIndex != INDEX_NONE)
		{
			Requests[Index].SetResult(ValidatedRequests[MappingIndex].GetResult());
		}
	}

	// For local storage we consider the push to have failed only if ALL backends gave an error, if at least one backend succeeded then the operation succeeded.
	// For persistent storage we require that all backends succeeded, so any errors will fail the push operation.
	return StorageType == EStorageType::Cache ? ErrorCount < Backends.Num() : ErrorCount == 0;
}

bool FVirtualizationManager::PullData(TArrayView<FPullRequest> Requests)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualizationManager::PullData);

	if (PullEnabledBackends.IsEmpty())
	{
		// TODO: See below, should errors here be fatal?
		UE_LOGF(LogVirtualization, Error, "Failed to pull payload(s) as there are no backends mounted!'");
		return false;
	}

	FConditionalScopeLock _(&DebugValues.ForceSingleThreadedCS, DebugValues.bSingleThreaded);

	BroadcastEvent(Requests, IVirtualizationSystem::PullBegunNotification);

	PullDataFromAllBackends(Requests);

	BroadcastEvent(Requests, IVirtualizationSystem::PullEndedNotification);

	bool bSuccess = true;
	for (const FPullRequest& Request : Requests)
	{
		// Report failed pulls of valid identifiers
		if (!Request.IsSuccess() && !Request.GetIdentifier().IsZero())
		{
			GetNotificationEvent().Broadcast(IVirtualizationSystem::PullFailedNotification, Request.GetIdentifier());

			// TODO: Maybe this should be a fatal error? If we keep it as an error we need to make sure any calling
			// code handles it properly.
			// Could be worth extending ::PullData to return error codes instead so we can make a better distinction 
			// between the payload not being found in any of the backends and one or more of the backends failing.
			UE_LOGF(LogVirtualization, Error, "Payload '%ls' failed to be pulled from any backend", *LexToString(Request.GetIdentifier()));

			bSuccess = false;
		}

		if (Request.IsSuccess())
		{
			checkf(Request.GetIdentifier() == Request.GetPayload().GetRawHash(), TEXT("Invalid payload for '%s'"), *LexToString(Request.GetIdentifier()));
		}
	}

	if (bSuccess)
	{
		UnattendedFailureMsgCount = 0;
	}

	return bSuccess;
}

EQueryResult FVirtualizationManager::QueryPayloadStatuses(TArrayView<const FIoHash> Ids, EStorageType StorageType, TArray<EPayloadStatus>& OutStatuses)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualizationManager::DoPayloadsExist);

	OutStatuses.SetNum(Ids.Num()); // Make sure we set the number out statuses before we potentially early out

	if (Ids.IsEmpty())
	{
		return EQueryResult::Success;
	}

	for (int32 Index = 0; Index < Ids.Num(); ++Index)
	{
		OutStatuses[Index] = Ids[Index].IsZero() ? EPayloadStatus::Invalid : EPayloadStatus::NotFound;
	}

	FBackendArray& Backends = StorageType == EStorageType::Cache ? CacheStorageBackends : PersistentStorageBackends;

	TArray<int8> HitCount;
	TArray<bool> Results;

	HitCount.SetNum(Ids.Num());
	Results.SetNum(Ids.Num());

	{
		FConditionalScopeLock _(&DebugValues.ForceSingleThreadedCS, DebugValues.bSingleThreaded);

		for (IVirtualizationBackend* Backend : Backends)
		{
			if (Backend->TryConnect(bAllowBackendReconnection) != IVirtualizationBackend::EConnectionStatus::Connected)
			{
				UE_LOGF(LogVirtualization, Verbose, "Cannot query backend  '%ls' as it was unable to connect", *Backend->GetDebugName());
				continue;
			}

			if (!Backend->DoPayloadsExist(Ids, Results))
			{
				// If a backend entirely failed we should early out and report the problem
				OutStatuses.Reset();
				return EQueryResult::Failure_Unknown;
			}

			for (int32 Index = 0; Index < Ids.Num(); ++Index)
			{
				if (!Ids[Index].IsZero() && Results[Index])
				{
					HitCount[Index]++;
				}
			}
		}
	}

	// Now we total up the hit count for each payload to see if it was found in none, all or some of the backends
	for (int32 Index = 0; Index < Ids.Num(); ++Index)
	{
		if (!Ids[Index].IsZero())
		{
			if (HitCount[Index] == 0)
			{
				OutStatuses[Index] = EPayloadStatus::NotFound;
			}
			else if (HitCount[Index] == Backends.Num())
			{
				OutStatuses[Index] = EPayloadStatus::FoundAll;
			}
			else
			{
				OutStatuses[Index] = EPayloadStatus::FoundPartial;
			}
		}
	}

	return EQueryResult::Success;
}

FVirtualizationResult FVirtualizationManager::TryVirtualizePackages(TConstArrayView<FString> PackagePaths, EVirtualizationOptions Options)
{
	FVirtualizationResult Result;

	if (IsEnabled() && IsPushingEnabled(EStorageType::Persistent))
	{
		const TArray<FString> SanitizedPaths = SanitizeFilePaths(PackagePaths);
		UE::Virtualization::VirtualizePackages(SanitizedPaths, Options, Result);

		if (Result.WasSuccessful() && !VirtualizationProcessTag.IsEmpty())
		{
			Result.DescriptionTags.Add(FText::FromString(VirtualizationProcessTag));
		}
	}

	return Result;
}

FRehydrationResult FVirtualizationManager::TryRehydratePackages(TConstArrayView<FString> PackagePaths, ERehydrationOptions Options)
{
	FRehydrationResult Result;

	const TArray<FString> SanitizedPaths = SanitizeFilePaths(PackagePaths);
	UE::Virtualization::RehydratePackages(SanitizedPaths, Options, Result);

	return Result;
}

ERehydrationResult FVirtualizationManager::TryRehydratePackages(TConstArrayView<FString> PackagePaths, uint64 PaddingAlignment, TArray<FText>& OutErrors, TArray<FSharedBuffer>& OutPackages, TArray<FRehydrationInfo>* OutInfo)
{
	OutErrors.Reset();

	UE::Virtualization::RehydratePackages(PackagePaths, PaddingAlignment, OutErrors, OutPackages, OutInfo);

	return OutErrors.IsEmpty() ? ERehydrationResult::Success : ERehydrationResult::Failed;
}

void FVirtualizationManager::DumpStats() const
{
#if ENABLE_COOK_STATS
	Profiling::LogStats();
#endif // ENABLE_COOK_STATS
}

FPayloadActivityInfo FVirtualizationManager::GetSystemStatistics() const
{
	FPayloadActivityInfo Info;

#if ENABLE_COOK_STATS
	for (const auto& Iterator : Profiling::CacheStats)
	{
		Info.Cache.PayloadCount += Iterator.Value.GetAccumulatedValueAnyThread(FCookStats::CallStats::EHitOrMiss::Hit, FCookStats::CallStats::EStatType::Counter);
		Info.Cache.TotalBytes += Iterator.Value.GetAccumulatedValueAnyThread(FCookStats::CallStats::EHitOrMiss::Hit, FCookStats::CallStats::EStatType::Bytes);
		Info.Cache.CyclesSpent += Iterator.Value.GetAccumulatedValueAnyThread(FCookStats::CallStats::EHitOrMiss::Hit, FCookStats::CallStats::EStatType::Cycles);
	}

	for (const auto& Iterator : Profiling::PushStats)
	{
		Info.Push.PayloadCount += Iterator.Value.GetAccumulatedValueAnyThread(FCookStats::CallStats::EHitOrMiss::Hit, FCookStats::CallStats::EStatType::Counter);
		Info.Push.TotalBytes += Iterator.Value.GetAccumulatedValueAnyThread(FCookStats::CallStats::EHitOrMiss::Hit, FCookStats::CallStats::EStatType::Bytes);
		Info.Push.CyclesSpent += Iterator.Value.GetAccumulatedValueAnyThread(FCookStats::CallStats::EHitOrMiss::Hit, FCookStats::CallStats::EStatType::Cycles);
	}

	for (const auto& Iterator : Profiling::PullStats)
	{
		Info.Pull.PayloadCount += Iterator.Value.GetAccumulatedValueAnyThread(FCookStats::CallStats::EHitOrMiss::Hit, FCookStats::CallStats::EStatType::Counter);
		Info.Pull.TotalBytes += Iterator.Value.GetAccumulatedValueAnyThread(FCookStats::CallStats::EHitOrMiss::Hit, FCookStats::CallStats::EStatType::Bytes);
		Info.Pull.CyclesSpent += Iterator.Value.GetAccumulatedValueAnyThread(FCookStats::CallStats::EHitOrMiss::Hit, FCookStats::CallStats::EStatType::Cycles);
	}
#endif // ENABLE_COOK_STATS

	return Info;
}

TArray<FBackendStats> FVirtualizationManager::GetBackendStatistics() const
{
	TArray<FBackendStats> Stats;
	
#if ENABLE_COOK_STATS

	Stats.Reserve(AllBackends.Num());

	for (const TUniquePtr<IVirtualizationBackend>& Backend : AllBackends)
	{
		FBackendStats& BackendStats = Stats.AddDefaulted_GetRef();

		BackendStats.DebugName = Backend->GetDebugName();
		BackendStats.ConfigName = Backend->GetConfigName();
		BackendStats.Type = PersistentStorageBackends.Contains(Backend.Get()) ? EStorageType::Persistent : EStorageType::Cache;

		FPayloadActivityInfo& Info = BackendStats.PayloadActivity;

		const FCookStats::CallStats& CacheStats = Profiling::GetCacheStats(*Backend);

		Info.Cache.PayloadCount = CacheStats.GetAccumulatedValueAnyThread(FCookStats::CallStats::EHitOrMiss::Hit, FCookStats::CallStats::EStatType::Counter);
		Info.Cache.TotalBytes = CacheStats.GetAccumulatedValueAnyThread(FCookStats::CallStats::EHitOrMiss::Hit, FCookStats::CallStats::EStatType::Bytes);
		Info.Cache.CyclesSpent = CacheStats.GetAccumulatedValueAnyThread(FCookStats::CallStats::EHitOrMiss::Hit, FCookStats::CallStats::EStatType::Cycles);

		const FCookStats::CallStats& PushStats = Profiling::GetPushStats(*Backend);

		Info.Push.PayloadCount = PushStats.GetAccumulatedValueAnyThread(FCookStats::CallStats::EHitOrMiss::Hit, FCookStats::CallStats::EStatType::Counter);
		Info.Push.TotalBytes = PushStats.GetAccumulatedValueAnyThread(FCookStats::CallStats::EHitOrMiss::Hit, FCookStats::CallStats::EStatType::Bytes);
		Info.Push.CyclesSpent = PushStats.GetAccumulatedValueAnyThread(FCookStats::CallStats::EHitOrMiss::Hit, FCookStats::CallStats::EStatType::Cycles);

		const FCookStats::CallStats& PullStats = Profiling::GetPullStats(*Backend);

		Info.Pull.PayloadCount = PullStats.GetAccumulatedValueAnyThread(FCookStats::CallStats::EHitOrMiss::Hit, FCookStats::CallStats::EStatType::Counter);
		Info.Pull.TotalBytes = PullStats.GetAccumulatedValueAnyThread(FCookStats::CallStats::EHitOrMiss::Hit, FCookStats::CallStats::EStatType::Bytes);
		Info.Pull.CyclesSpent = PullStats.GetAccumulatedValueAnyThread(FCookStats::CallStats::EHitOrMiss::Hit, FCookStats::CallStats::EStatType::Cycles);
	}
#endif // ENABLE_COOK_STATS

	return Stats;
}

void FVirtualizationManager::ApplySettingsFromConfigFiles(const FConfigFile& ConfigFile)
{
	UE_LOGF(LogVirtualization, Display, "Loading virtualization manager settings from config files...");

	const TCHAR* LegacyConfigSection = TEXT("Core.ContentVirtualization");
	const TCHAR* ConfigSection = TEXT("Core.VirtualizationModule");

	// Note that many options are doubled up as we are moving the options for this module from "Core.ContentVirtualization"
	// to it's own specific "Core.VirtualizationModule" section. This duplication can be removed before we ship 5.1
	
	// The backend graph is the most important value and so should come first for logging purposes
	{
		if (!ConfigFile.GetString(LegacyConfigSection, TEXT("BackendGraph"), BackendGraphName))
		{
			ConfigFile.GetString(ConfigSection, TEXT("BackendGraph"), BackendGraphName);
		}

		UE_LOGF(LogVirtualization, Display, "\tBackendGraphName : %ls", *BackendGraphName);
	}

	{
		// This value was moved from Core.ContentVirtualization to Core.VirtualizationModule then renamed from
		// 'EnablePushToBackend' to 'EnablePayloadVirtualization' so there are a few paths we need to cover here.
		// This can also be cleaned up for 5.1 shipping.
		if (ConfigFile.GetBool(LegacyConfigSection, TEXT("EnablePushToBackend"), bAllowPackageVirtualization))
		{
			UE_LOGF(LogVirtualization, Warning, "\tFound legacy ini file setting [Core.ContentVirtualization].EnablePushToBackend, rename to [Core.VirtualizationModule].EnablePayloadVirtualization");
		}
		else if (ConfigFile.GetBool(ConfigSection, TEXT("EnablePushToBackend"), bAllowPackageVirtualization))
		{
			UE_LOGF(LogVirtualization, Warning, "\tFound legacy ini file setting [Core.VirtualizationModule].EnablePushToBackend, rename to [Core.VirtualizationModule].EnablePayloadVirtualization");
		}
		else
		{
			ConfigFile.GetBool(ConfigSection, TEXT("EnablePayloadVirtualization"), bAllowPackageVirtualization);
		}

		UE_LOGF(LogVirtualization, Display, "\tEnablePayloadVirtualization : %ls", bAllowPackageVirtualization ? TEXT("true") : TEXT("false"));
	}

	{
		bool bCacheOnPull = true;
		if (ConfigFile.GetBool(LegacyConfigSection, TEXT("EnableCacheAfterPull"), bCacheOnPull) ||
			ConfigFile.GetBool(ConfigSection, TEXT("EnableCacheAfterPull"), bCacheOnPull))
		{
			UE_LOGF(LogVirtualization, Warning, "\tEnableCacheAfterPull is deprecated, replace with 'EnableCacheOnPull=True|False'");
		}
		else
		{
			ConfigFile.GetBool(ConfigSection, TEXT("EnableCacheOnPull"), bCacheOnPull);
		}

		if (!bCacheOnPull)
		{
			EnumRemoveFlags(CachingPolicy, ECachingPolicy::CacheOnPull);
		}

		UE_LOGF(LogVirtualization, Display, "\tEnableCacheOnPull : %ls", EnumHasAllFlags(CachingPolicy, ECachingPolicy::CacheOnPull) ? TEXT("true") : TEXT("false"));
	}

	{
		bool bCacheOnPush = true;
		ConfigFile.GetBool(ConfigSection, TEXT("EnableCacheOnPush"), bCacheOnPush);

		if (!bCacheOnPush)
		{
			EnumRemoveFlags(CachingPolicy, ECachingPolicy::CacheOnPush);
		}
		
		UE_LOGF(LogVirtualization, Display, "\tEnableCacheOnPush : %ls",  EnumHasAllFlags(CachingPolicy, ECachingPolicy::CacheOnPush) ? TEXT("true") : TEXT("false"));
	}

	{
		if (!ConfigFile.GetInt64(LegacyConfigSection, TEXT("MinPayloadLength"), MinPayloadLength))
		{
			ConfigFile.GetInt64(ConfigSection, TEXT("MinPayloadLength"), MinPayloadLength);
		}

		UE_LOG(LogVirtualization, Display, TEXT("\tMinPayloadLength : %" INT64_FMT), MinPayloadLength);
	}

	{
		ConfigFile.GetString(ConfigSection, TEXT("VirtualizationProcessTag"), VirtualizationProcessTag);
		UE_LOGF(LogVirtualization, Display, "\tVirtualizationProcessTag : %ls", *VirtualizationProcessTag);
	}

	{
		FString FilterModeString;
		if (ConfigFile.GetString(LegacyConfigSection, TEXT("FilterMode"), FilterModeString) ||
			ConfigFile.GetString(ConfigSection, TEXT("FilterMode"), FilterModeString))
		{
			if (!LexTryParseString(FilteringMode, FilterModeString))
			{
				UE_LOGF(LogVirtualization, Error, "[Core.VirtualizationModule].FilterMode was an invalid value! Allowed: 'OptIn'|'OptOut' Found '%ls'", *FilterModeString);
			}
		}

		UE_LOGF(LogVirtualization, Display, "\tFilterMode : %ls", LexToString(FilteringMode));
	}

	{
		ConfigFile.GetBool(ConfigSection, TEXT("FilterMapContent"), bFilterMapContent);
		UE_LOGF(LogVirtualization, Display, "\tFilterMapContent : %ls", bFilterMapContent ? TEXT("true") : TEXT("false"));
	}


	TArray<FString> DisabledAssetTypesFromIni;
	if (ConfigFile.GetArray(LegacyConfigSection, TEXT("DisabledAsset"), DisabledAssetTypesFromIni) > 0 ||
		ConfigFile.GetArray(ConfigSection, TEXT("DisabledAsset"), DisabledAssetTypesFromIni) > 0)
	{
		UE_LOGF(LogVirtualization, Display, "\tVirtualization is disabled for payloads of the following assets:");
		DisabledAssetTypes.Reserve(DisabledAssetTypesFromIni.Num());
		for(const FString& AssetType : DisabledAssetTypesFromIni)
		{ 
			UE_LOGF(LogVirtualization, Display, "\t\t%ls", *AssetType);
			DisabledAssetTypes.Add(FName(AssetType));
		}	
	}

	{
		ConfigFile.GetBool(ConfigSection, TEXT("AllowSubmitIfVirtualizationFailed"), bAllowSubmitIfVirtualizationFailed);
		UE_LOGF(LogVirtualization, Display, "\tAllowSubmitIfVirtualizationFailed : %ls", bAllowSubmitIfVirtualizationFailed ? TEXT("true") : TEXT("false"));
	}

#if UE_VIRTUALIZATION_CONNECTION_LAZY_INIT == 0
	ConfigFile.GetBool(ConfigSection, TEXT("LazyInitConnections"), bLazyInitConnections);

	if (bLazyInitConnections)
	{
		// Check if we should override the value of bLazyInitConnections depending on if the current process is interactive or not
		bool bDisableLazyInitIfInteractive = false;
		ConfigFile.GetBool(ConfigSection, TEXT("DisableLazyInitIfInteractive"), bDisableLazyInitIfInteractive);

		const bool bIsUsingSlate = UE_VA_WITH_SLATE != 0;
		const bool bIsUnattended = FApp::IsUnattended();
		const bool bIsRunningCommandlet = IsRunningCommandlet();

		const bool bIsInteractive = bIsUsingSlate && !bIsUnattended && !bIsRunningCommandlet;

		if (bDisableLazyInitIfInteractive && bIsInteractive)
		{
			bLazyInitConnections = false;
		}
	}

	UE_LOGF(LogVirtualization, Display, "\tLazyInitConnections : %ls", bLazyInitConnections ? TEXT("true") : TEXT("false"));
#else
	bLazyInitConnections = true;
	UE_LOGF(LogVirtualization, Display, "\tLazyInitConnections : true (set by code)");
#endif //UE_VIRTUALIZATION_CONNECTION_LAZY_INIT

	{
		ConfigFile.GetBool(ConfigSection, TEXT("UseLegacyErrorHandling"), bUseLegacyErrorHandling);
		UE_LOGF(LogVirtualization, Display, "\tUseLegacyErrorHandling : %ls", bUseLegacyErrorHandling ? TEXT("true") : TEXT("false"));
	}

	{
		ConfigFile.GetString(ConfigSection, TEXT("PullErrorAdditionalMsg"), PullErrorAdditionalMsg);
		// This value is not echoed to the log file, as seeing an error string there might confuse users
	}
	
	{
		ConfigFile.GetString(ConfigSection, TEXT("ConnectionHelpUrl"), ConnectionHelpUrl);
		// This value is not echoed to the log file
	}

	{
		ConfigFile.GetBool(ConfigSection, TEXT("ForceCachingOnPull"), bForceCachingOnPull);
		UE_LOGF(LogVirtualization, Display, "\tForceCachingOnPull : %ls", bForceCachingOnPull ? TEXT("true") : TEXT("false"));
	}

	{
		ConfigFile.GetBool(ConfigSection, TEXT("AllowBackendReconnection"), bAllowBackendReconnection);
		UE_LOGF(LogVirtualization, Display, "\tAllowBackendReconnection : %ls", bAllowBackendReconnection ? TEXT("true") : TEXT("false"));
	}

	{
		ConfigFile.GetInt(ConfigSection, TEXT("UnattendedRetryTimer"), UnattendedRetryTimer);
		ConfigFile.GetInt(ConfigSection, TEXT("UnattendedRetryCount"), UnattendedRetryCount);
		
		if (ShouldRetryWhenUnattended())
		{
			UE_LOGF(LogVirtualization, Display, "\tUnattendedRetryTimer : %d retries every %d(s)", UnattendedRetryCount, UnattendedRetryTimer);
		}
		else
		{
			UE_LOGF(LogVirtualization, Display, "\tUnattendedRetries : disabled");
		}
	}

	// Deprecated
	{
		bool bDummyValue = true;
		if (ConfigFile.GetBool(LegacyConfigSection, TEXT("FilterEngineContent"), bDummyValue) ||
			ConfigFile.GetBool(ConfigSection, TEXT("FilterEngineContent"), bDummyValue))
		{
			UE_LOGF(LogVirtualization, Error, "\tFilterEngineContent is now deprecated (engine content is never virtualized) please update before compatibility is removed in UE 5.9");
		}

		if (ConfigFile.GetBool(LegacyConfigSection, TEXT("FilterEnginePluginContent"), bDummyValue) ||
			ConfigFile.GetBool(ConfigSection, TEXT("FilterEnginePluginContent"), bDummyValue))
		{
			UE_LOGF(LogVirtualization, Error, "\tFilterEnginePluginContent is now deprecated (engine content is never virtualized) please update before compatibility is removed in UE 5.9");
		}
	}

	// Check for any legacy settings and print them out (easier to do this in one block rather than one and time)
	{
		// Entries that are allows to be in [Core.ContentVirtualization]
		static const TArray<FString> AllowedEntries = { TEXT("SystemName") , TEXT("LazyInit"), TEXT("InitPreSlate") };
		
		TArray<FString> LegacyEntries;	
		if (const FConfigSection* LegacySection = ConfigFile.FindSection(LegacyConfigSection))
		{
			for (const TPair<FName, FConfigValue>& It : *LegacySection)
			{
				FString Name = It.Key.ToString();
			
				if (!AllowedEntries.Contains(Name))
				{
					LegacyEntries.Add(MoveTemp(Name));
				}
			}
		}

		if (!LegacyEntries.IsEmpty())
		{
			UE_LOGF(LogVirtualization, Error, "\tFound %d legacy ini file settings under [Core.ContentVirtualization] that should be moved to [Core.VirtualizationModule], please update before compatibility is removed in UE 5.9", LegacyEntries.Num());
			for (const FString& LegacyEntry : LegacyEntries)
			{
				UE_LOGF(LogVirtualization, Error, "\t\t%ls", *LegacyEntry);
			}
		}
	}
}

void FVirtualizationManager::ApplySettingsFromFromCmdline()
{
	if (!bLazyInitConnections && IsCmdLineParamSet(TEXT("VALazyInitConnections"), TEXT("VA-LazyInitConnections")))
	{
		bLazyInitConnections = true;
		UE_LOGF(LogVirtualization, Display, "Cmdline has set the virtualization system backends to lazy init their connections");
	}

	if (bAllowPackageVirtualization && IsCmdLineParamSet(TEXT("VASkipPkgVirtualization"), TEXT("VA-SkipPkgVirtualization")))
	{
		bAllowPackageVirtualization = false;
		UE_LOGF(LogVirtualization, Warning, "The virtualization process has been disabled via the command line");
	}
}

void FVirtualizationManager::ApplySettingsFromCVar()
{
	if (!bLazyInitConnections && CVarLazyInitConnections.GetValueOnAnyThread())
	{
		bLazyInitConnections = true;
		UE_LOGF(LogVirtualization, Display, "CVar has set the virtualization system backends to lazy init their connections");
	}
}

void FVirtualizationManager::ApplyDebugSettingsFromFromCmdline()
{
	if (IsCmdLineParamSet(TEXT("VASingleThreaded"), TEXT("VA-SingleThreaded")))
	{
		DebugValues.bSingleThreaded = true;
		UE_LOGF(LogVirtualization, Warning, "Cmdline has set the virtualization system to run single threaded");
	}

	if (IsCmdLineParamSet(TEXT("VAValidatePushes"), TEXT("VA-ValidatePushes")))
	{
		DebugValues.bValidateAfterPush = true;
		UE_LOGF(LogVirtualization, Warning, "Cmdline has set the virtualization system to pull each payload after pushing to either local or persistent storage");
	}

	FString CmdlineGraphName;
	if (IsCmdLineValueSet(TEXT("-VABackendGraph="), TEXT("-VA-BackendGraph="), CmdlineGraphName))
	{
		UE_LOGF(LogVirtualization, Display, "Backend graph overridden from the cmdline: '%ls'", *CmdlineGraphName);
		BackendGraphName = CmdlineGraphName;
	}

	FString MissOptions;
	if (IsCmdLineValueSet(TEXT("-VAMissBackends="), TEXT("-VA-MissBackends="), MissOptions))
	{
		MissOptions.ParseIntoArray(DebugValues.MissBackends, TEXT("+"), true);

		UE_LOGF(LogVirtualization, Warning, "Cmdline has disabled payload pulling for the following backends:");
		for (const FString& Backend : DebugValues.MissBackends)
		{
			UE_LOGF(LogVirtualization, Warning, "\t%ls", *Backend);
		}
	}

	DebugValues.MissChance = 0.0f;
	if (IsCmdLineValueSet(TEXT("-VAMissChance="), TEXT("-VA-MissChance="), DebugValues.MissChance))
	{
		DebugValues.MissChance = FMath::Clamp(DebugValues.MissChance, 0.0f, 100.0f);

		UE_LOGF(LogVirtualization, Warning, "Cmdline has set a %.1f%% chance of a payload pull failing", DebugValues.MissChance);
	}
}

void FVirtualizationManager::RegisterConsoleCommands()
{
	{
#if UE_USE_GLOBAL_CVAR
		IConsoleVariable* Handle = IConsoleManager::Get().FindConsoleVariable(TEXT("VA.AllowPkgVirtualization"));

		if (Handle != nullptr)
		{
			auto Callback = [this](IConsoleVariable* CVar)
			{
				this->bAllowPackageVirtualization = CVar->GetBool();

				if (this->bAllowPackageVirtualization)
				{
					UE_LOGF(LogVirtualization, Display, "The virtualization process has been enabled via the cvar 'VA.SkipPkgVirtualization'");
				}
				else
				{
					UE_LOGF(LogVirtualization, Display, "The virtualization process has been disabled via the cvar 'VA.SkipPkgVirtualization'");
				}
			};

			FDelegateHandle CallbackHandle = Handle->OnChangedDelegate().AddLambda(MoveTemp(Callback));

			if (bAllowPackageVirtualization != Handle->GetBool())
			{
				Callback(Handle);
			}

			DebugValues.ConsoleDelegateHandles.Add({ Handle, CallbackHandle });
		}
		else
		{
			UE_LOGF(LogVirtualization, Warning, "CVar VA.AllowPkgVirtualization could not be found and will not function");
		}
#else
		bool bOriginalValue = bAllowPackageVirtualization;

		IConsoleVariable* Handle = IConsoleManager::Get().RegisterConsoleVariableRef(
			TEXT("VA.AllowPkgVirtualization"),
			bAllowPackageVirtualization,
			TEXT("When true submitting packages in the editor will no longer trigger the virtualization process")
		);

		auto Callback = [](IConsoleVariable* CVar)
			{
				if (CVar->GetBool())
				{
					UE_LOGF(LogVirtualization, Display, "The virtualization process has been enabled via the cvar 'VA.SkipPkgVirtualization'");
				}
				else
				{
					UE_LOGF(LogVirtualization, Display, "The virtualization process has been disabled via the cvar 'VA.SkipPkgVirtualization'");
				}
			};
		
		// Log the change if the cvar was modified on the commandline
		if (bOriginalValue != bAllowPackageVirtualization)
		{
			Callback(Handle);
		}

		Handle->OnChangedDelegate().AddLambda(MoveTemp(Callback));

		DebugValues.ConsoleObjects.Add(Handle);
#endif // UE_USE_GLOBAL_CVAR
	}

	DebugValues.ConsoleObjects.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("VA.MissBackends"),
		TEXT("A debug command which can be used to disable payload pulling on one or more backends"),
		FConsoleCommandWithArgsAndOutputDeviceDelegate::CreateRaw(this, &FVirtualizationManager::OnUpdateDebugMissBackendsFromConsole)
	));

	DebugValues.ConsoleObjects.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("VA.MissChance"),
		TEXT("A debug command which can be used to set the chance that a payload pull will fail"),
		FConsoleCommandWithArgsAndOutputDeviceDelegate::CreateRaw(this, &FVirtualizationManager::OnUpdateDebugMissChanceFromConsole)
	));

	DebugValues.ConsoleObjects.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("VA.MissCount"),
		TEXT("A debug command which can be used to cause the next X number of payload pulls to fail"),
		FConsoleCommandWithArgsAndOutputDeviceDelegate::CreateRaw(this, &FVirtualizationManager::OnUpdateDebugMissCountFromConsole)
	));

	DebugValues.ConsoleObjects.Add(IConsoleManager::Get().RegisterConsoleVariableRef(
		TEXT("VA.SingleThreaded"),
		DebugValues.bSingleThreaded,
		TEXT("When set the asset virtualization system will only access backends in a single threaded manner")
	));

	DebugValues.ConsoleObjects.Add(IConsoleManager::Get().RegisterConsoleVariableRef(
		TEXT("VA.ValidatePushes"),
		DebugValues.bValidateAfterPush,
		TEXT("When set the asset virtualization system will pull each payload after pushing to either local or persistent storage")
	));
}

void FVirtualizationManager::OnUpdateDebugMissBackendsFromConsole(const TArray<FString>& Args, FOutputDevice& OutputDevice)
{
	if (Args.IsEmpty())
	{
		OutputDevice.Log(TEXT("VA.MissBackends command help"));
		OutputDevice.Log(TEXT("This command allows you to disable the pulling of payloads by specific backends"));
		OutputDevice.Log(TEXT(""));
		OutputDevice.Log(TEXT("Commands:"));
		OutputDevice.Log(TEXT("VA.MissBackends reset            - Empties the list of backends, everything will function normally"));
		OutputDevice.Log(TEXT("VA.MissBackends list             - Prints the list of backends affected"));
		OutputDevice.Log(TEXT("VA.MissBackends set Name0 Name1  - List each backend that you want to fail to pull payloads"));
		OutputDevice.Log(TEXT("VA.MissBackends set All          - All backends will fail to pull payloads"));
		OutputDevice.Log(TEXT(""));
		OutputDevice.Log(TEXT("Valid backend names:"));

		for (const TUniquePtr<IVirtualizationBackend>& Backend : AllBackends)
		{
			OutputDevice.Logf(TEXT("\t%s"), *Backend->GetConfigName());
		}
	}
	else if (Args.Num() == 1)
	{
		if (Args[0] == TEXT("reset"))
		{
			DebugValues.MissBackends.Empty();
			UpdateBackendDebugState();
		}
		else if (Args[0] == TEXT("list"))
		{
			if (!DebugValues.MissBackends.IsEmpty())
			{
				OutputDevice.Log(TEXT("Disabled backends:"));
				for (const FString& Backend : DebugValues.MissBackends)
				{
					OutputDevice.Logf(TEXT("\t%s"), *Backend);
				}
			}
			else
			{
				OutputDevice.Log(TEXT("No backends are disabled"));
			}
		}
		else
		{
			OutputDevice.Log(ELogVerbosity::Error, TEXT("Invalid args for the VA.MissBackends command!"));
		}
	}
	else if (Args[0] == TEXT("set"))
	{	
		DebugValues.MissBackends.Empty(Args.Num() - 1);

		for (int32 Index = 1; Index < Args.Num(); ++Index)
		{
			DebugValues.MissBackends.Add(Args[Index]);
		}

		UpdateBackendDebugState();
	}
	else
	{
		OutputDevice.Log(ELogVerbosity::Error, TEXT("Invalid args for the VA.MissBackends command!"));
	}
}

void FVirtualizationManager::OnUpdateDebugMissChanceFromConsole(const TArray<FString>& Args, FOutputDevice& OutputDevice)
{
	if (Args.IsEmpty())
	{
		OutputDevice.Log(TEXT("VA.MissChance command help"));
		OutputDevice.Log(TEXT("This command allows you to set the chance (in percent) that a payload pull request will just fail"));
		OutputDevice.Log(TEXT(""));
		OutputDevice.Log(TEXT("Commands:"));
		OutputDevice.Log(TEXT("VA.MissChance show     - prints the current miss percent chance"));
		OutputDevice.Log(TEXT("VA.MissChance set Num - Sets the miss percent chance to the given value"));
	}
	else if (Args.Num() == 1 && Args[0] == TEXT("show"))
	{
		OutputDevice.Logf(TEXT("Current debug miss chance: %.1f%%"), DebugValues.MissChance);
	}
	else if (Args.Num() == 2 && Args[0] == TEXT("set"))
	{
		if (::LexTryParseString(DebugValues.MissChance, *Args[1]))
		{
			DebugValues.MissChance = FMath::Clamp(DebugValues.MissChance, 0.0f, 100.0f);
			OutputDevice.Logf(TEXT("Current debug miss chance set to %.1f%%"), DebugValues.MissChance);
		}
		else
		{
			DebugValues.MissChance = 0.0f;
			OutputDevice.Log(ELogVerbosity::Error, TEXT("Invalid value, current debug miss chance reset to 0.0%"));
		}
	}
	else
	{
		OutputDevice.Log(ELogVerbosity::Error, TEXT("Invalid args for the VA.MissChance command!"));
	}
}

void FVirtualizationManager::OnUpdateDebugMissCountFromConsole(const TArray<FString>& Args, FOutputDevice& OutputDevice)
{
	if (Args.IsEmpty())
	{
		OutputDevice.Log(TEXT("VA.MissCount command help"));
		OutputDevice.Log(TEXT("This command allows you to set the next X number of payload pulls to fail"));
		OutputDevice.Log(TEXT(""));
		OutputDevice.Log(TEXT("Commands:"));
		OutputDevice.Log(TEXT("VA.MissCount show     - prints the current number of future payload pulls that will fail"));
		OutputDevice.Log(TEXT("VA.MissChance set Num - Sets the number of future payload pulls to fail"));
	}
	else if (Args.Num() == 1 && Args[0] == TEXT("show"))
	{
		// DebugMissCount could end up negative if many threads are pulling at once, so clamp to 0 as the min value
		const int32 Value = FMath::Max(DebugValues.MissCount.load(std::memory_order_relaxed), 0);
		OutputDevice.Logf(TEXT("The next '%d' payload pulls will fail"), Value);
	}
	else if (Args.Num() == 2 && Args[0] == TEXT("set"))
	{
		int32 ValueToSet = 0;
		if (::LexTryParseString(ValueToSet, *Args[1]))
		{
			DebugValues.MissCount.store(ValueToSet, std::memory_order_relaxed);
			OutputDevice.Logf(TEXT("The next '%d' payload pulls have been set to fail"), ValueToSet);
		}
		else
		{
			DebugValues.MissCount.store(0, std::memory_order_relaxed);
			OutputDevice.Log(ELogVerbosity::Error, TEXT("Invalid value, the number of future payload pulls to fail has been set to zero"));
		}
	}
	else
	{
		OutputDevice.Log(ELogVerbosity::Error, TEXT("Invalid args for the VA.MissCount command!"));
	}
}

void FVirtualizationManager::UpdateBackendDebugState()
{
	for (TUniquePtr<IVirtualizationBackend>& Backend : AllBackends)
	{
		const bool bDisable = ShouldDebugDisablePulling(Backend->GetConfigName());
		Backend->SetOperationDebugState(IVirtualizationBackend::EOperations::Pull, bDisable);
	}
}

bool FVirtualizationManager::ShouldDebugDisablePulling(FStringView BackendConfigName) const
{
	if (DebugValues.MissBackends.IsEmpty())
	{
		return false;
	}

	if (DebugValues.MissBackends[0] == TEXT("All"))
	{
		return true;
	}

	for (const FString& Name : DebugValues.MissBackends)
	{
		if (Name == BackendConfigName)
		{
			return true;
		}
	}

	return false;
}

bool FVirtualizationManager::ShouldDebugFailPulling()
{
	// We don't want to decrement on every function call to avoid DebugMissCount
	// underflowing, so we only try to decrement if the count is positive.
	// It doesn't really matter if the value ends up a little bit negative.
	if (DebugValues.MissCount.load(std::memory_order_relaxed) > 0)
	{
		if (DebugValues.MissCount.fetch_sub(1, std::memory_order_relaxed) > 0)
		{
			return true;
		}
	}

	if (DebugValues.MissChance == 0.0f)
	{
		return false;
	}
	else
	{
		// Could consider adding a lock here, although FRandomStream
		// is thread safe, many threads hitting it could cause a few
		// threads to get the same results.
		// Since this is a debug function and the percent is only a
		// rough guide, adding a lock is considered overkill. This
		// should only be done if in the future we decide that we want
		// more accuracy.
		static FRandomStream RandomStream(NAME_None);

		const float RandValue = RandomStream.FRand() * 100.0f;
		return RandValue <= DebugValues.MissChance;
	}
}

void FVirtualizationManager::MountBackends(const FConfigFile& ConfigFile)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualizationManager::MountBackends);

	const FRegistedFactories FactoryLookupTable = FindBackendFactories();
	UE_LOGF(LogVirtualization, Verbose, "Found %d backend factories", FactoryLookupTable.Num());

	const TCHAR* GraphName = *BackendGraphName;

	if(!ConfigFile.DoesSectionExist(GraphName))
	{
		UE_LOGF(LogVirtualization, Fatal, "Unable to find the backend graph: '%ls' [ini=%ls].", GraphName, *GEngineIni);
	}

	UE_LOGF(LogVirtualization, Display, "Mounting virtualization backend graph: '%ls'", GraphName);

	// It is important to parse the local storage hierarchy first so those backends will show up before the
	// persistent storage backends in 'PullEnabledBackends'.
	ParseHierarchy(ConfigFile, GraphName, TEXT("CacheStorageHierarchy"), TEXT("LocalStorageHierarchy"), FactoryLookupTable, CacheStorageBackends);
	ParseHierarchy(ConfigFile, GraphName, TEXT("PersistentStorageHierarchy"), nullptr, FactoryLookupTable, PersistentStorageBackends);

	// Apply and disabled backends from the command line
	UpdateBackendDebugState();
}

void FVirtualizationManager::ParseHierarchy(const FConfigFile& ConfigFile, const TCHAR* GraphName, const TCHAR* HierarchyKey, const TCHAR* LegacyHierarchyKey, const FRegistedFactories& FactoryLookupTable, FBackendArray& PushArray)
{
	bool bFoundHierarchy = false;
	FString HierarchyData;

	if (LegacyHierarchyKey != nullptr && ConfigFile.GetValue(GraphName, LegacyHierarchyKey, HierarchyData))
	{
		UE_LOGF(LogVirtualization, Warning, "\tFound legacy HierarchyKey '%ls', rename to '%ls'", LegacyHierarchyKey, HierarchyKey);
		bFoundHierarchy = true;
	}
	else if (ConfigFile.GetValue(GraphName, HierarchyKey, HierarchyData))
	{
		bFoundHierarchy = true;
	}

	if (bFoundHierarchy)
	{
		if (HierarchyData.IsEmpty())
		{
			UE_LOGF(LogVirtualization, Fatal, "\tThe '%ls' entry for backend graph '%ls' is empty [ini=%ls].", HierarchyKey, GraphName, *GEngineIni);
		}

		const TArray<FString> Entries = ParseEntries(HierarchyData);

		UE_LOGF(LogVirtualization, Display, "\tThe '%ls' has %d backend(s)", HierarchyKey, Entries.Num());

		for (const FString& Entry : Entries)
		{
			CreateBackend(ConfigFile, GraphName, Entry, FactoryLookupTable, PushArray);
		}	
	}
	else
	{
		UE_LOGF(LogVirtualization, Display, "\tNo entries for '%ls' in the content virtualization backend graph '%ls' [ini=%ls].", HierarchyKey, GraphName, *GEngineIni);
	}
}

bool FVirtualizationManager::CreateBackend(const FConfigFile& ConfigFile, const TCHAR* GraphName, const FString& ConfigEntryName, const FRegistedFactories& FactoryLookupTable, FBackendArray& PushArray)
{
	// All failures in this method are considered fatal, however it still returns true/false in case we decide
	// to be more forgiving in the future.
	UE_LOGF(LogVirtualization, Display, "Mounting backend entry '%ls'", *ConfigEntryName);

	FString BackendData;
	if (!ConfigFile.GetValue(GraphName, *ConfigEntryName, BackendData))
	{
		UE_LOGF(LogVirtualization, Fatal, "Unable to find the entry '%ls' in the content virtualization backend graph '%ls' [ini=%ls].", *ConfigEntryName, GraphName, *GEngineIni);
		return false;
	}

	FString BackendType;
	if (FParse::Value(*BackendData, TEXT("Type="), BackendType) && !BackendType.IsEmpty())
	{
		// Put the rest of the ini file entry into a string to pass to the backend.
		FString Cmdine = BackendData.RightChop(BackendData.Find(BackendType) + BackendType.Len());
		Cmdine.RemoveFromEnd(TEXT(")"));

		UE::Virtualization::IVirtualizationBackendFactory* const* FactoryPtr = FactoryLookupTable.Find(FName(BackendType));
		if (FactoryPtr != nullptr && *FactoryPtr != nullptr)
		{
			IVirtualizationBackendFactory* Factory = *FactoryPtr;
			TUniquePtr<IVirtualizationBackend> Backend = Factory->CreateInstance(ProjectName, ConfigEntryName);

			if (Backend == nullptr)
			{
				UE_LOGF(LogVirtualization, Fatal, "IVirtualizationBackendFactory '%ls' failed to create an instance!", *Factory->GetName().ToString());
				return false;
			}

			if (Backend->Initialize(Cmdine))
			{
				// The read only flag can be applied to any backend so we check for it and apply it at this point
				bool bReadOnly = false;
				FParse::Bool(*Cmdine, TEXT("ReadOnly="), bReadOnly);

				if (bReadOnly && Backend->DisableOperation(IVirtualizationBackend::EOperations::Push))
				{
					UE_LOGF(LogVirtualization, Display, "The backend '%ls' was set to readonly by the config file!", *Backend->GetDebugName());
				}

				AddBackend(MoveTemp(Backend), PushArray);
			}
			else
			{
				UE_LOGF(LogVirtualization, Fatal, "Backend '%ls' reported errors when initializing", *ConfigEntryName);
				return false;
			}
		}
		else
		{
			UE_LOGF(LogVirtualization, Fatal, "No backend factory found that can create the type '%ls'", *BackendType);
			return false;
		}
	}
	else
	{
		UE_LOGF(LogVirtualization, Fatal, "No 'Type=' entry found for '%ls' in the config file", *ConfigEntryName);
		return false;
	}

	return true;
}

void FVirtualizationManager::AddBackend(TUniquePtr<IVirtualizationBackend> Backend, FBackendArray& PushArray)
{
	checkf(!AllBackends.Contains(Backend), TEXT("Adding the same virtualization backend (%s) multiple times!"), *Backend->GetDebugName());

	// Move ownership of the backend to AllBackends
	AllBackends.Add(MoveTemp(Backend));

	// Get a reference pointer to use in the other backend arrays
	IVirtualizationBackend* BackendRef = AllBackends.Last().Get();
	check(BackendRef != nullptr);

	if (BackendRef->IsOperationSupported(IVirtualizationBackend::EOperations::Pull))
	{
		PullEnabledBackends.Add(BackendRef);
	}

	if (BackendRef->IsOperationSupported(IVirtualizationBackend::EOperations::Push))
	{
		PushArray.Add(BackendRef);
	}

	// We immediately try to connect once the backend has been added.
	// In the future this will be made async to avoid blocking the GameThread on startup
	if (!bLazyInitConnections)
	{
		BackendRef->TryConnect(bAllowBackendReconnection);
	}

	COOK_STAT(Profiling::CreateStats(*BackendRef));
}

bool FVirtualizationManager::IsPersistentBackend(IVirtualizationBackend& Backend)
{
	return PersistentStorageBackends.Contains(&Backend);
}

void FVirtualizationManager::CachePayloads(TArrayView<FPushRequest> Requests, const IVirtualizationBackend* BackendSource, IVirtualizationBackend::EPushFlags Flags)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualizationManager::CachePayload);

	//check(!bPendingBackendConnections);
	// TODO: Connection?

	// We start caching at the first (assumed to be fastest) local cache backend. 
	for (IVirtualizationBackend* BackendToCache : CacheStorageBackends)
	{
		// We stop once we reach the backend that the payloads were first pulled from
		if (BackendToCache == BackendSource)
		{
			return;
		}

		// Reset any previous caching results
		for (FPushRequest& Request : Requests)
		{
			Request.ResetResult();
		}	

		const bool bResult = TryCacheDataToBackend(*BackendToCache, Requests, Flags);

		if (!bResult)
		{
			for (const FPushRequest& Request : Requests)
			{
				UE_CLOGF(Request.GetResult().WasError(), LogVirtualization, Warning, "Failed to cache payload '%ls' to backend '%ls'",
					*LexToString(Request.GetIdentifier()),
					*BackendToCache->GetDebugName());
			}
		}

		// Debug operation to validate that the payload we just cached can be retrieved from storage
		if (DebugValues.bValidateAfterPush && bResult && BackendToCache->IsOperationSupported(IVirtualizationBackend::EOperations::Pull))
		{
			for (const FPushRequest& Request : Requests)
			{
				FText Errors;
				FPullRequest ValidationRequest(Request.GetIdentifier());
				PullDataFromBackend(*BackendToCache, MakeArrayView(&ValidationRequest, 1), Errors);

				checkf(Request.GetPayload().GetRawHash() == ValidationRequest.GetPayload().GetRawHash(),
					TEXT("[%s] Failed to pull payload '%s' after it was cached to backend"),
					*BackendToCache->GetDebugName(),
					*LexToString(Request.GetIdentifier()));
			}
		}
	}
}

bool FVirtualizationManager::TryCacheDataToBackend(IVirtualizationBackend& Backend, TArrayView<FPushRequest> Requests, IVirtualizationBackend::EPushFlags Flags)
{
	COOK_STAT(FCookStats::CallStats & Stats = Profiling::GetCacheStats(Backend));
	COOK_STAT(FCookStats::FScopedStatsCounter Timer(Stats));
	COOK_STAT(Timer.TrackCyclesOnly());
	
	if (Backend.PushData(Requests, Flags))
	{
#if ENABLE_COOK_STATS
		const bool bIsInGameThread = IsInGameThread();
		bool bWasDataPushed = false;

		for (const FPushRequest& Request : Requests)
		{
			if (Request.GetResult().WasPushed())
			{
				Stats.Accumulate(FCookStats::CallStats::EHitOrMiss::Hit, FCookStats::CallStats::EStatType::Counter, 1l, bIsInGameThread);
				Stats.Accumulate(FCookStats::CallStats::EHitOrMiss::Hit, FCookStats::CallStats::EStatType::Bytes, Request.GetPayload().GetCompressedSize(), bIsInGameThread);

				bWasDataPushed = true;

				UE_LOGF(LogVirtualization, Verbose, "[%ls] Cached payload '%ls'", *Backend.GetDebugName(), *LexToString(Request.GetIdentifier()));
			}
		}

		if (bWasDataPushed)
		{
			Timer.AddHit(0);
		}
#endif // ENABLE_COOK_STATS

		return true;
	}
	else
	{
		return false;
	}
}

bool FVirtualizationManager::TryPushDataToBackend(IVirtualizationBackend& Backend, TArrayView<FPushRequest> Requests)
{
	COOK_STAT(FCookStats::CallStats & Stats = Profiling::GetPushStats(Backend));
	COOK_STAT(FCookStats::FScopedStatsCounter Timer(Stats));
	COOK_STAT(Timer.TrackCyclesOnly());

	const bool bPushResult = Backend.PushData(Requests, IVirtualizationBackend::EPushFlags::None);

#if ENABLE_COOK_STATS
	if (bPushResult)
	{
		Timer.AddHit(0);

		const bool bIsInGameThread = IsInGameThread();

		for (const FPushRequest& Request : Requests)
		{
			if (Request.GetResult().WasPushed())
			{
				Stats.Accumulate(FCookStats::CallStats::EHitOrMiss::Hit, FCookStats::CallStats::EStatType::Counter, 1l, bIsInGameThread);	
				Stats.Accumulate(FCookStats::CallStats::EHitOrMiss::Hit, FCookStats::CallStats::EStatType::Bytes, Request.GetPayloadSize(), bIsInGameThread);
			}
		}
	}
#endif // ENABLE_COOK_STATS
	
	return bPushResult;
}

void FVirtualizationManager::PullDataFromAllBackends(TArrayView<FPullRequest> Requests)
{
	if (ShouldDebugFailPulling())
	{
		UE_LOGF(LogVirtualization, Verbose, "Debug miss chance (%.1f%%) invoked", DebugValues.MissChance);
		return;
	}

	FPullRequestCollection RequestsCollection(Requests);
	
	while(true)
	{
		TStringBuilder<512> BackendErrors;

		for (IVirtualizationBackend* Backend : PullEnabledBackends)
		{
			check(Backend != nullptr);

			if (Backend->IsOperationDebugDisabled(IVirtualizationBackend::EOperations::Pull))
			{
				UE_LOGF(LogVirtualization, Verbose, "Pulling from backend '%ls' is debug disabled", *Backend->GetDebugName());
				continue;
			}

			if (Backend->TryConnect(bAllowBackendReconnection) != IVirtualizationBackend::EConnectionStatus::Connected)
			{
				UE_LOGF(LogVirtualization, Verbose, "Cannot pull from backend '%ls' as it was unable to connect", *Backend->GetDebugName());
				continue;
			}

			FText Errors;
			PullDataFromBackend(*Backend, RequestsCollection.GetRequests(), Errors);

			// We only want to report errors from persistent backends, cached backends are allowed to fail.
			if (!Errors.IsEmpty() && IsPersistentBackend(*Backend))
			{
				if (BackendErrors.Len() != 0)
				{
					BackendErrors << LINE_TERMINATOR;
				}

				BackendErrors << Backend->GetDebugName() << TEXT(": ") << Errors.ToString();
			}

			const bool bShouldCache = EnumHasAllFlags(CachingPolicy, ECachingPolicy::CacheOnPull);

			TArray<FPushRequest> PayloadsToCache = RequestsCollection.OnPullCompleted(*Backend, bShouldCache);
			if (!PayloadsToCache.IsEmpty())
			{
				const IVirtualizationBackend::EPushFlags CacheOnPullFlags = bForceCachingOnPull	? IVirtualizationBackend::EPushFlags::Force
																								: IVirtualizationBackend::EPushFlags::None;

				CachePayloads(PayloadsToCache, Backend, CacheOnPullFlags);
			}

			// We can early out if there is no more requests to make
			if (RequestsCollection.IsWorkComplete())
			{
				break;
			}
		}

		if (RequestsCollection.IsWorkComplete())
		{
			return; // All payloads pulled
		}
		else if (OnPayloadPullError(RequestsCollection, BackendErrors) == ErrorHandlingResult::AcceptFailedPayloads)
		{
			return; // Some payloads failed to pull but we will let the calling code deal with that.
					// This path should only be taken if legacy error handling is enabled.
		}

		// The user opted to retry pulling the payloads that failed.
	}
}

void FVirtualizationManager::PullDataFromBackend(IVirtualizationBackend& Backend, TArrayView<FPullRequest> Requests, FText& OutErrors)
{
	COOK_STAT(FCookStats::CallStats & Stats = Profiling::GetPullStats(Backend));
	COOK_STAT(FCookStats::FScopedStatsCounter Timer(Stats));
	COOK_STAT(Timer.TrackCyclesOnly());

	Backend.PullData(Requests, IVirtualizationBackend::EPullFlags::None, OutErrors);

#if ENABLE_COOK_STATS
	const bool bIsInGameThread = IsInGameThread();

	for (const FPullRequest& Request : Requests)
	{
		Timer.AddHit(0);

		if (Request.IsSuccess())
		{
			Stats.Accumulate(FCookStats::CallStats::EHitOrMiss::Hit, FCookStats::CallStats::EStatType::Counter, 1l, bIsInGameThread);
			Stats.Accumulate(FCookStats::CallStats::EHitOrMiss::Hit, FCookStats::CallStats::EStatType::Bytes, Request.GetPayload().GetCompressedSize(), bIsInGameThread);
		}
	}
#endif //ENABLE_COOK_STATS
}

FVirtualizationManager::ErrorHandlingResult FVirtualizationManager::OnPayloadPullError(const FPullRequestCollection& Requests, FStringView BackendErrors) const
{
	if (bUseLegacyErrorHandling)
	{
		return ErrorHandlingResult::AcceptFailedPayloads;
	}

	// Note that when handling failed pull errors there are several code paths that allow the process or the user to retry to the failed
	// pull operation in case it was caused by a temporary infrastructure issue. So until we are certain that no more retrying will be
	// attempted we must log everything as a 'warning' rather than an 'error' as many CIS systems will treat even single logged error as
	// a reason to consider a job failed.

	for (const FIoHash& FailedPayload : Requests.GetFailedPayloads())
	{
		UE_LOGF(LogVirtualization, Warning, "Failed to pull payload '%ls'", *LexToString(FailedPayload));
	}

	static FCriticalSection CriticalSection;

	// Only allow a single thread to make an active error report. The most likely cause of an error is an infrastructure outage which would
	// cause all threads making requests to fail at the same time as we don't want to spam the user with dialogs. If they (or the unattended
	// process) requests a retry then we should apply that to all threads currently encountering issues at the same time.
	if (CriticalSection.TryLock())
	{
		const FText Title(LOCTEXT("VAPullTitle", "Failed to pull virtualized data!"));

		FTextBuilder MsgBuilder;
		MsgBuilder.AppendLine(LOCTEXT("VAPullMsgHeader", "Failed to pull payload(s) from virtualization storage and allowing the editor to continue could corrupt data!"));

		if (!BackendErrors.IsEmpty())
		{
			MsgBuilder.AppendLine(FString(TEXT("")));
			MsgBuilder.AppendLine(FString(BackendErrors));
		}

		if (!PullErrorAdditionalMsg.IsEmpty())
		{
			MsgBuilder.AppendLine(FString(TEXT("")));
			MsgBuilder.AppendLine(PullErrorAdditionalMsg);
		}

		// By default we should quit the process unless the user can opt to retry
		EAppReturnType::Type Result = EAppReturnType::No;

		if (Utils::IsProcessInteractive())
		{
			MsgBuilder.AppendLine(FString(TEXT("")));
			MsgBuilder.AppendLine(LOCTEXT("VAPullMsgYes", "[Yes] Retry pulling the data"));
			MsgBuilder.AppendLine(LOCTEXT("VAPullMsgNo", "[No] Quit the editor"));

			const FText Message = MsgBuilder.ToText();

			FDisableUnattendedScriptGlobal DisableScope;
			Result = FMessageDialog::Open(EAppMsgType::YesNo, EAppReturnType::No, Message, Title);
		}
		else
		{
			const FText Message = MsgBuilder.ToText();

			if (!ShouldRetryWhenUnattended() || UnattendedFailureMsgCount >= UnattendedRetryCount)
			{
				UE_LOGF(LogVirtualization, Error, "%ls", *Message.ToString());
			}
			else
			{
				// Only log as a warning while we are retrying so that if we do recover out logging will
				// not cause the running process to be considered a failure due to an error.
				UE_LOGF(LogVirtualization, Warning, "Failed to pull from any backend on attempt (%d/%d)", UnattendedFailureMsgCount.load(), UnattendedRetryCount);

				Result = EAppReturnType::Yes; // Attempt to retry the failed pull

				if (UnattendedRetryTimer > 0)
				{
					UE_LOGF(LogVirtualization, Warning, "Waiting for %d seconds before trying to pull again...", UnattendedRetryTimer);
					FPlatformProcess::SleepNoStats(static_cast<float>(UnattendedRetryTimer));
				}

				UnattendedFailureMsgCount++;
			}
		}

		{
			TArray<FAnalyticsEventAttribute> Attributes;
			Attributes.Add({ TEXT("UserSelection"), Result == EAppReturnType::No ? TEXT("Quit") : TEXT("Retry")});

			// Need to flush if we are going to quit to make sure that the analytics payloads are sent properly.
			const EAnalyticsFlags Flags = Result == EAppReturnType::No ? EAnalyticsFlags::Flush : EAnalyticsFlags::None;

			GetAnalyticsRecordEvent().Broadcast(TEXT("Editor.VA.PayloadPullError"), Attributes, Flags);
		}

		if (Result == EAppReturnType::No)
		{
			// Reporting a fatal error will print the callstack and initiate the crash handling system to be treated like a bug
			// where as this error indicates an infrastructure failure or other connection issue which needs to be solved.
			// So we will force the process to close after logging our errors instead.
			UE_LOGF(LogVirtualization, Error, "Failed to pull payloads from persistent storage and the connection could not be re-established, exiting...");
			GIsCriticalError = 1;
			FPlatformMisc::RequestExit(true);
		}

		CriticalSection.Unlock();
	}
	else
	{
		// Since we failed to get the lock the error is being dealt with on another thread so we need to wait here until
		// that thread has resolved the problem or terminated the process.
		FScopeLock _(&CriticalSection);
	}

	return ErrorHandlingResult::Retry;
}

bool FVirtualizationManager::ShouldVirtualizeAsset(const UObject* OwnerObject) const
{
	if (OwnerObject == nullptr)
	{
		return true;
	}

	const UClass* OwnerClass = OwnerObject->GetClass();
	if (OwnerClass == nullptr)
	{
		// TODO: Not actually sure if the class being nullptr is reasonable or if we should warn/error here?
		return true;
	}

	const FName ClassName = OwnerClass->GetFName();
	return DisabledAssetTypes.Find(ClassName) == nullptr;
}

bool FVirtualizationManager::ShouldVirtualizePackage(const FPackagePath& PackagePath) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualizationManager::ShouldVirtualizePackage);

	// We require a valid mounted path for filtering
	if (!PackagePath.IsMountedPath())
	{
		return true;
	}

	TStringBuilder<256> PackageName;
	PackagePath.AppendPackageName(PackageName);

	TStringBuilder<64> MountPointName;
	TStringBuilder<256> MountPointPath;
	TStringBuilder<256> RelativePath;

	if (!FPackageName::TryGetMountPointForPath(PackageName, MountPointName, MountPointPath, RelativePath))
	{
		return true;
	}

	// Do not virtualize engine content
	if (MountPointName.ToView() == TEXT("/Engine/") || FPaths::IsUnderDirectory(MountPointPath.ToString(), FPaths::EnginePluginsDir()))
	{
		return false;
	}

	const UVirtualizationFilterSettings* Settings = GetDefault<UVirtualizationFilterSettings>();
	if (Settings != nullptr)
	{
		auto DoesMatch = [](const TArray<FString>& Paths, const FStringView& PackagePath) -> bool
		{
			for (const FString& PathToMatch : Paths)
			{
				if (PathToMatch.EndsWith(TEXT("/")))
				{
					// Directory path, exclude everything under it
					if (PackagePath.StartsWith(PathToMatch))
					{
						return true;
					}
				}
				else
				{
					// Path to an asset, exclude if it matches exactly
					if (PackagePath == PathToMatch)
					{
						return true;
					}
				}
			}

			return false;
		};

		const FStringView PackageNameView = PackageName.ToView();

		if (DoesMatch(Settings->ExcludePackagePaths, PackageNameView))
		{
			return false;
		}

		if (DoesMatch(Settings->IncludePackagePaths, PackageNameView))
		{
			return true;
		}
	}
	
	// The package is not in any of the include/exclude paths so we use the default behavior
	return ShouldVirtualizeAsDefault();
}

bool FVirtualizationManager::ShouldVirtualize(FStringView Context) const
{
	// First see if we can convert the context from a raw string to a valid package path.
	// If we can extract a package path then we should use the package filtering code
	// path instead.
	FPackagePath PackagePath;
	if (FPackagePath::TryFromPackageName(Context, PackagePath))
	{
		return ShouldVirtualizePackage(PackagePath);
	}

	if (FPackagePath::TryFromMountedName(Context, PackagePath))
	{
		return ShouldVirtualizePackage(PackagePath);
	}

	// The package is not in any of the include/exclude paths so we use the default behavior
	return ShouldVirtualizeAsDefault();
}

bool FVirtualizationManager::ShouldVirtualizeAsDefault() const
{
	switch (FilteringMode)
	{
		case EPackageFilterMode::OptOut:
			return true;
		case EPackageFilterMode::OptIn:
			return false;
		default:
			checkNoEntry();
			return false;
	}
}

bool FVirtualizationManager::ShouldRetryWhenUnattended() const
{
	return UnattendedRetryCount > 0;
}

void FVirtualizationManager::BroadcastEvent(TConstArrayView<FPullRequest> Requests, ENotification Event)
{
	for (const FPullRequest& Request : Requests)
	{
		GetNotificationEvent().Broadcast(IVirtualizationSystem::PullEndedNotification, Request.GetIdentifier());
	}
}

void FVirtualizationManager::GatherAnalytics(TArray<FAnalyticsEventAttribute>& Attributes) const
{
	Attributes.Reserve(Attributes.Num() + 10);

	FPayloadActivityInfo PayloadActivityInfo = GetSystemStatistics();

	const FString BaseName = TEXT("Virtualization");

	{
		FString AttrName = BaseName + TEXT("_Enabled");
		Attributes.Emplace(MoveTemp(AttrName), IsEnabled());
	}

	{
		FString AttrName = BaseName + TEXT("_Cache_TimeSpent");
		Attributes.Emplace(MoveTemp(AttrName), (double)PayloadActivityInfo.Cache.CyclesSpent * FPlatformTime::GetSecondsPerCycle());
	}

	{
		FString AttrName = BaseName + TEXT("_Cache_PayloadCount");
		Attributes.Emplace(MoveTemp(AttrName), (double)PayloadActivityInfo.Cache.PayloadCount);
	}

	{
		FString AttrName = BaseName + TEXT("_Cache_TotalBytes");
		Attributes.Emplace(MoveTemp(AttrName), (double)PayloadActivityInfo.Cache.TotalBytes);
	}

	{
		FString AttrName = BaseName + TEXT("_Push_TimeSpent");
		Attributes.Emplace(MoveTemp(AttrName), (double)PayloadActivityInfo.Push.CyclesSpent * FPlatformTime::GetSecondsPerCycle());
	}

	{
		FString AttrName = BaseName + TEXT("_Push_PayloadCount");
		Attributes.Emplace(MoveTemp(AttrName), (double)PayloadActivityInfo.Push.PayloadCount);
	}

	{
		FString AttrName = BaseName + TEXT("_Push_TotalBytes");
		Attributes.Emplace(MoveTemp(AttrName), (double)PayloadActivityInfo.Push.TotalBytes);
	}

	{
		FString AttrName = BaseName + TEXT("_Pull_TimeSpent");
		Attributes.Emplace(MoveTemp(AttrName), (double)PayloadActivityInfo.Pull.CyclesSpent * FPlatformTime::GetSecondsPerCycle());
	}

	{
		FString AttrName = BaseName + TEXT("_Pull_PayloadCount");
		Attributes.Emplace(MoveTemp(AttrName), (double)PayloadActivityInfo.Pull.PayloadCount);
	}

	{
		FString AttrName = BaseName + TEXT("_Pull_TotalBytes");
		Attributes.Emplace(MoveTemp(AttrName), (double)PayloadActivityInfo.Pull.TotalBytes);
	}
}

EPayloadFilterReason FVirtualizationManager::FixFilterFlags(FStringView PackagePath, uint64 SizeOnDisk, EPayloadFilterReason CurrentFilterFlags)
{
	// We only apply new filters if the payload does not currently have any as stored filtered reasons would be checked first before
	// we ever got to this part of the virtualization process.
	if (CurrentFilterFlags == EPayloadFilterReason::None)
	{
		if (static_cast<int64>(SizeOnDisk) < MinPayloadLength)
		{
			return EPayloadFilterReason::MinSize;
		}

		if (!ShouldVirtualize(PackagePath))
		{
			return EPayloadFilterReason::Path;
		}
	}

	return CurrentFilterFlags;
}

FString FVirtualizationManager::GetConnectionHelpUrl()
{
	return ConnectionHelpUrl;
}

} // namespace UE::Virtualization

#undef UE_INLINE_ALLOCATION_COUNT
#undef LOCTEXT_NAMESPACE
