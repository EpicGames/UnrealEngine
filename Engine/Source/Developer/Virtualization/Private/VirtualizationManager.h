// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compression/CompressedBuffer.h"
#include "HAL/CriticalSection.h"
#include "IVirtualizationBackend.h"
#include "Logging/LogMacros.h"
#include "Templates/UniquePtr.h"
#include "Virtualization/VirtualizationSystem.h"

class IConsoleObject;
class FOutputDevice;
struct FAnalyticsEventAttribute;

/**
 * Configuring the backend hierarchy
 * 
 * The [Core.ContentVirtualization] section can contain a string 'BackendGraph' which will set with the name of  
 * the backend graph, if not set then the default 'ContentVirtualizationBackendGraph_None' will be used instead. 
 * This value can also be overridden from the command line by using 'BackendGraph=FooBar' where FooBar is the 
 * name of the graph.
 * 
 * The first entry in the graph to be parsed will be the 'Hierarchy' which describes which backends should be
 * mounted and in which order. For example 'Hierarchy=(Entry=Foo, Entry=Bar)' which should mount two backends
 * 'Foo' and 'Bar' in that order. 
 * 
 * Each referenced backend in the hierarchy will then require it's own entry in the graph where the key will be
 * it's name in the hierarchy and the value a string describing how to set it up. 
 * The value must contain 'Type=X' where X is the name used to find the correct IVirtualizationBackendFactory 
 * to create the backend with. 
 * Once the backend is created then reset of the string will be passed to it, so that additional customization
 * can be extracted. Depending on the backend implementation these values may or may not be required.
 * 
 * Example graph:
 * [ContentVirtualizationBackendGraph_Example] 
 * Hierarchy=(Entry=MemoryCache, Entry=NetworkShare)
 * MemoryCache=(Type=InMemory)
 * NetworkShare=(Type=FileSystem, Path="\\path\to\somewhere")
 * 
 * The graph is named 'ContentVirtualizationBackendGraph_Example'.
 * The hierarchy contains two entries 'InMemory' and 'NetworkShare' to be mounted in that order
 * MemoryCache creates a backend of type 'InMemory' and has no additional customization
 * NetworkShare creates a backend of type 'FileSystem' and provides an additional path, the filesystem backend would 
 * fatal error without this value.
 */

/**
 * Filtering
 * 
 * By default all packages in a project can be virtualized once the system has been enabled. The can be
 * overridden by the filtering system, either by excluding specific packages/directories, or by changing
 * the default so that no package will be virtualized except packages/directories that have been 
 * specifically marked as to be virtualized.
 * 
 * Note that the filtering is applied when a package is saved and stored as meta data in the FPackageTrailer.
 * This means that you can scan your package files and reason about the behavior of the payloads but also
 * means that if you change your project filtering rules that packages have to be re-saved in order for the
 * filtering to be applied.
 * 
 * @see ShouldVirtualizePackage or ShouldVirtualize for implementation details.
 * 
 * Basic Setup:
 * 
 * [Core.VirtualizationModule]
 * FilterMode=OptIn/OptOut					The general mode to be applied to all packages. With 'OptIn' packages will
 *											not be virtualized unless their path is included via VirtualizationFilterSettings.
 *											With 'OptOut' all packages will be virtualized unless excluded via 
 *											VirtualizationFilterSettings [Default=OptOut]
 * FilterMapContent=True/False				When true any payload stored in a .umap or _BuildData.uasset file will be
 *											excluded from virtualization [Default=true]
 * 
 * PackagePath Setup:
 * 
 * In addition to the default filtering mode set above, payloads stored in packages can be filtered based on the
 * package path. This allows a package to be including in the virtualization process or excluded from it.
 * 
 * Note that these paths will be stored in the ini files under the Saved directory. To remove a path make sure to 
 * use the - syntax to remove the entry from the array, rather than removing the line itself. Otherwise it will
 * persist until the saved config file has been reset.
 *
 * [/Script/Virtualization.VirtualizationFilterSettings]
 * +ExcludePackagePaths="/MountPoint/PathToExclude/"				Excludes any package found under '/MountPoint/PathToExclude/' from the virtualization process
 * +ExcludePackagePaths="/MountPoint/PathTo/ThePackageToExclude"	Excludes the specific package '/MountPoint/PathTo/ThePackageToExclude' from the virtualization process
 * +IncludePackagePaths="/MountPoint/PathToInclude/"				Includes any package found under '/MountPoint/PathToInclude/' in the virtualization process
 * +IncludePackagePaths="/MountPoint/PathTo/ThePackageToInclude"	Includes the specific package '/MountPoint/PathTo/ThePackageToInclude' in the virtualization process
 */

/*
 * FVirtualizationManager
 * 
 * Ini file setup:
 * 
 * EnablePayloadVirtualization [bool]:			When true the virtualization process will be enabled (usually when a package is submitted
												to revision control. [Default=true]
 * EnableCacheOnPull [bool]:					When true payloads will be pushed to cached storage after being pulled from persistent
 *												storage. [Default=true]
 * EnableCacheOnPush [bool]:					When true payloads will be pushed to cached storage right before being pushed to persistent
 *												storage. [Default=true]
 * MinPayloadLength  [int64]:					The minimum length (in bytes) that a payload must reach before it can be considered for
 *												virtualization. Use this to strike a balance between disk space and the number of smaller
												payloads in your project being virtualized. [Default=0]
 * BackendGraph [string]:						The name of the backend graph to use. The default graph has no backends and effectively
												disables the system. It is expected that a project will define the graph that it wants
												and then set this option [Default=ContentVirtualizationBackendGraph_None]
 * VirtualizationProcessTag [string]:			The tag to be applied to any set of packages that have had  the virtualization process run
 *												on them. Typically this means appending the tag to the description of a changelist of 
 *												packages. This value can be set to an empty string. [Default="#virtualized"]
 * AllowSubmitIfVirtualizationFailed [bool]:	Revision control submits that trigger the virtualization system can either allow or block
 *												the submit if the virtualization process fails based on this value. True will allow a
 *												submit with an error to continue and false will block the submit. Note that by error we mean
 *												that the packages were not virtualized, not that bad data was produced. [Default=false]
 * LazyInitConnections [bool]:					When true, backends will not attempt to connect to their services until actually required.
 *												This can remove lengthy connection steps from the process init phase and then only connect
 *												if we actually need that service. Note that if this is true then the connection can come from
 *												any thread, so custom backend code will need to take that into account. [Default=false]
 * DisableLazyInitIfInteractive [bool]			When true 'LazyInitConnections' will be forced to false if slate is enabled. This exists because
 *												some backends can show slate dialogs when their initial connection fails to prompt for the 
 *												correct login values. When 'LazyInitConnections' is true, this request can come on any thread and
 *												trying to marshal the slate request to the gamethread can often introduce thread locks. Setting
 *												both this and 'LazyInitConnections' to true will allow tools that do not use slate to initialize
 *												the VA connections on use, but force tools that can display the slate dialog to initialized 
 *												during preinit on the game thread so that the dialog can be shown. Note that this only overrides
												the setting of 'LazyInitConnections' via the config file, not cvar, cmdline or code [Default=false]
 * UseLegacyErrorHandling [bool]:				Controls how we deal with errors encountered when pulling payloads. When true a failed payload 
 *												pull will return an error and allow the process to carry on (the original error handling logic)
 *												and when false a dialog will be displayed to the user warning them about the failed pull and 
 *												prompting them to retry the pull or to quit the process. [Default=false]
 * PullErrorAdditionalMsg [string]				An additional message that will be added to the error dialog presented on payload pull failure.
 *												This allows you to add custom information, such as links to internal help docs without editing
 *												code. Note that this additional message only works with the error dialog and will do nothing
 *												if 'UseLegacyErrorHandling' is true. [Default=""]
 * ForceCachingOnPull [bool]:					When true backends will be told to always upload the payload when a caching as a result of 
 *												a payload pull as in this scenario we already know that the backend failed to pull the payload
 *												before it was pulled from a backend later in the hierarchy. Can be used to try and skip
 *												expensive existence checks, or if a backend is in a bad state where it believes it has the payload
 *												but is unable to actually return the data. [Default=false]
 * AllowBackendReconnection [bool]:				When true backends that have previously failed to connect to their services can retry to connect
 *												each time they are accessed. When false the legacy behavior is invoked and backends may only attempt
 *												to connect a single time and after that will be unusable. [Default=true]
 * UnattendedRetryCount [int32]:				How many times the process should retry pulling payloads after a failure is encountered if the
												process is unattended. Usually when a payload pull fails we ask the user to try and fix the issue
												and retry, but in unattended mode we just log an error and terminate the process. In some cases
												such as build machines with unreliable internet it is possible that the process could recover in
												which case setting this value might help. Zero or negative values will disable the system. 
												Note: If many pulls are occurring at the same time on many threads a very short network outage might
												spawn many errors in which case we try to group these errors into a single 'try' so 32 errors on 32
												threads would not immediately blow past a retry count of 30 for example. [Default=0]
 * UnattendedRetryTimer [int32]					If 'UnattendedRetryCount' is set to a positive value then this value sets how long (in seconds)
 *												the process should wait after a failure is encountered before retrying the pull. Depending on the
												likely cause of the failure you may want to set this value to several minutes. [Default=0]
 */

namespace UE::Virtualization
{
class FPullRequestCollection;

/** The default mode of filtering to use with package paths that do not match entries in UVirtualizationFilterSettings */
enum class EPackageFilterMode : uint8
{
	/** Packages will be virtualized by default and must be opted out by the use of UVirtualizationFilterSettings::ExcludePackagePaths */
	OptOut = 0,
	/** Packages will not be virtualized by default and must be opted in by the user of UVirtualizationFilterSettings::IncludePackagePaths */
	OptIn
};

/** Attempt to convert a string buffer to EPackageFilterMode */
bool LexTryParseString(EPackageFilterMode& OutValue, FStringView Buffer);

/** This is used as a wrapper around the various potential back end implementations. 
	The calling code shouldn't need to care about which back ends are actually in use. */
class FVirtualizationManager final : public IVirtualizationSystem
{
public:
	using FRegistedFactories = TMap<FName, IVirtualizationBackendFactory*>;
	using FBackendArray = TArray<IVirtualizationBackend*>;

	FVirtualizationManager();
	virtual ~FVirtualizationManager();

private:
	/* IVirtualizationSystem implementation */

	virtual bool Initialize(const FInitParams& InitParams) override;

	virtual bool IsEnabled() const override;
	virtual bool IsPushingEnabled(EStorageType StorageType) const override;

	virtual EPayloadFilterReason FilterPayload(const UObject* Owner) const override;

	virtual bool AllowSubmitIfVirtualizationFailed() const override;
	
	virtual bool PushData(TArrayView<FPushRequest> Requests, EStorageType StorageType) override;
	virtual bool PullData(TArrayView<FPullRequest> Requests) override;

	virtual EQueryResult QueryPayloadStatuses(TArrayView<const FIoHash> Ids, EStorageType StorageType, TArray<EPayloadStatus>& OutStatuses) override;

	virtual FVirtualizationResult TryVirtualizePackages(TConstArrayView<FString> PackagePaths, EVirtualizationOptions Options) override;

	virtual FRehydrationResult TryRehydratePackages(TConstArrayView<FString> PackagePaths, ERehydrationOptions Options) override;
	virtual ERehydrationResult TryRehydratePackages(TConstArrayView<FString> PackagePaths, uint64 PaddingAlignment, TArray<FText>& OutErrors, TArray<FSharedBuffer>& OutPackages, TArray<FRehydrationInfo>* OutInfo) override;

	virtual void DumpStats() const override;

	virtual FPayloadActivityInfo GetSystemStatistics() const override;

	virtual TArray<FBackendStats> GetBackendStatistics() const override;

	virtual void GatherAnalytics(TArray<FAnalyticsEventAttribute>& Attributes) const override;

	virtual FOnNotification& GetNotificationEvent() override
	{
		return NotificationEvent;
	}

public:
	/** @see UE::Virtualization::Utils::FixFilterFlags */
	EPayloadFilterReason FixFilterFlags(FStringView PackagePath, uint64 SizeOnDisk, EPayloadFilterReason CurrentFilterFlags);
	
private:

	void ApplySettingsFromConfigFiles(const FConfigFile& ConfigFile);
	void ApplySettingsFromFromCmdline();
	void ApplySettingsFromCVar();

	void ApplyDebugSettingsFromFromCmdline();

	void RegisterConsoleCommands();

	void OnUpdateDebugMissBackendsFromConsole(const TArray<FString>& Args, FOutputDevice& OutputDevice);
	void OnUpdateDebugMissChanceFromConsole(const TArray<FString>& Args, FOutputDevice& OutputDevice);
	void OnUpdateDebugMissCountFromConsole(const TArray<FString>& Args, FOutputDevice& OutputDevice);
	void UpdateBackendDebugState();

	bool ShouldDebugDisablePulling(FStringView BackendConfigName) const;	
	bool ShouldDebugFailPulling();

	void MountBackends(const FConfigFile& ConfigFile);
	void ParseHierarchy(const FConfigFile& ConfigFile, const TCHAR* GraphName, const TCHAR* HierarchyKey, const TCHAR* LegacyHierarchyKey, const FRegistedFactories& FactoryLookupTable, FBackendArray& PushArray);
	bool CreateBackend(const FConfigFile& ConfigFile, const TCHAR* GraphName, const FString& ConfigEntryName, const FRegistedFactories& FactoryLookupTable, FBackendArray& PushArray);

	void AddBackend(TUniquePtr<IVirtualizationBackend> Backend, FBackendArray& PushArray);

	bool IsPersistentBackend(IVirtualizationBackend& Backend);

	void CachePayloads(TArrayView<FPushRequest> Requests, const IVirtualizationBackend* BackendSource, IVirtualizationBackend::EPushFlags Flags);

	bool TryCacheDataToBackend(IVirtualizationBackend& Backend, TArrayView<FPushRequest> Requests, IVirtualizationBackend::EPushFlags Flags);
	bool TryPushDataToBackend(IVirtualizationBackend& Backend, TArrayView<FPushRequest> Requests);

	void PullDataFromAllBackends(TArrayView<FPullRequest> Requests);
	void PullDataFromBackend(IVirtualizationBackend& Backend, TArrayView<FPullRequest> Requests, FText& OutErrors);

	enum class ErrorHandlingResult
	{
		/** We should try to pull the failed payloads again */
		Retry = 0,
		/** We should accept that some of the payloads failed and leave it to the calling system to handle */
		AcceptFailedPayloads
	};

	ErrorHandlingResult OnPayloadPullError(const FPullRequestCollection& Requests, FStringView BackendErrors) const;
	
	bool ShouldVirtualizeAsset(const UObject* Owner) const;

	/** 
	 * Determines if a package path should be virtualized or not based on any exclusion/inclusion patterns
	 * that might have been set in UVirtualizationFilterSettings.
	 * If the path does not match any pattern set in UVirtualizationFilterSettings then use the default 
	 * FilterMode to determine if the payload should be virtualized or not.
	 * 
	 * @param PackagePath	The path of the package to check. This can be empty which would indicate that
	 *						a payload is not owned by a specific package.
	 * @return				True if the package should be virtualized and false if the package path is 
	 *						excluded by the projects current filter set up.
	 */
	bool ShouldVirtualizePackage(const FPackagePath& PackagePath) const;

	/**
	 * Determines if a package should be virtualized or not based on the given content.
	 * If the context can be turned into a package path then ::ShouldVirtualizePackage 
	 * will be used instead.
	 * If the context is not a package path then we use the default FilterMode to determine
	 * if the payload should be virtualized or not.
	 * 
	 * @return True if the context should be virtualized and false if not.
	 */
	bool ShouldVirtualize(FStringView Context) const;

	/** Determines if the default filtering behavior is to virtualize a payload or not */
	bool ShouldVirtualizeAsDefault() const;

	/** Returns if the process will attempt to retry a failed pull when the process is unattended mode */
	bool ShouldRetryWhenUnattended() const;

	void BroadcastEvent(TConstArrayView<FPullRequest> Ids, ENotification Event);

private:
	// The following members are set from the config file

	/** Are packages allowed to be virtualized when submitted to source control. Defaults to true. */
	bool bAllowPackageVirtualization;

	enum ECachingPolicy
	{
		/** Never push payloads to cached storage */
		None = 0,
		/** Cache payloads after they have been pulled from persistent storage */
		CacheOnPull = 1 << 0,
		/** Cache payloads right before they are pushed to persistent storage */
		CacheOnPush = 1 << 1,

		AlwaysCache = CacheOnPull | CacheOnPush
	};

	FRIEND_ENUM_CLASS_FLAGS(ECachingPolicy);

	/** A bitfield describing when we push payloads to cached storage. */
	ECachingPolicy CachingPolicy;

	/** The minimum length for a payload to be considered for virtualization. Defaults to 0 bytes. */
	int64 MinPayloadLength;

	/** The name of the backend graph to load from the config ini file that will describe the backend hierarchy */
	FString BackendGraphName;

	/** The tag that will be returned when the virtualization process has run, commonly used to post fix changelist descriptions */
	FString VirtualizationProcessTag;

	/** The default filtering mode to apply if a payload is not matched with an option in UVirtualizationFilterSettings */
	EPackageFilterMode FilteringMode;

	/** Should payloads in .umap files (or associated _BuildData files) be filtered out and never virtualized */
	bool bFilterMapContent;

	/** Should file submits be allowed to continue if a call to TryVirtualizePackages fails */
	bool bAllowSubmitIfVirtualizationFailed;
	
	/** Should backends defer connecting to their services until first use */
	bool bLazyInitConnections;

	/** When true we do not display an error dialog on failed payload pulling and rely on the caller handling it */
	bool bUseLegacyErrorHandling;

	/** When true IVirtualizationBackend::EPushFlags::Force will be passed to backends that need to cache payloads when pulling */
	bool bForceCachingOnPull;

	/** When true backends will be able to try to reconnect to their service's if previous connection attempts failed */
	bool bAllowBackendReconnection;

	/** An additional error message to display when pulling payloads fails */
	FString PullErrorAdditionalMsg;

	/** Optional url used to augment connection failure error messages */
	static FString ConnectionHelpUrl;

	/** The number of times to retry pulling when errors are encountered in an unattended process, values <0 disable the system */
	int32 UnattendedRetryCount = 0;

	/** The how long (in seconds) to wait after payload pulling errors before retrying. Does nothing if 'UnattendedRetryCount' is disabled */
	int32 UnattendedRetryTimer = 0;

private:

	/** The name of the current project */
	FString ProjectName;

	/** The names of all asset types that should not virtualize. See @IsDisabledForObject */
	TSet<FName> DisabledAssetTypes;

	/** All of the backends that were mounted during graph creation */
	TArray<TUniquePtr<IVirtualizationBackend>> AllBackends;

	/** Backends used for caching operations (must support push operations). */
	FBackendArray CacheStorageBackends; 

	/** Backends used for persistent storage operations (must support push operations). */
	FBackendArray PersistentStorageBackends; 

	/** 
	 * The hierarchy of backends to pull from, this is assumed to be ordered from fastest to slowest
	 * and can contain a mixture of local cacheable and persistent backends 
	 */
	FBackendArray PullEnabledBackends;

	/** Our notification Event */
	FOnNotification NotificationEvent;

	/** Track how many times we've displayed a message about failed payload pulls since the last successful pull (only used in unattended mode)*/
	mutable std::atomic<int32> UnattendedFailureMsgCount = 0;

	// Members after this point at used for debugging operations only!

	struct FDebugValues
	{
		/** All of the console commands/variables that we register, so they can be unregistered when the manager is destroyed */
		TArray<IConsoleObject*> ConsoleObjects;

		/** 
		 * Contains all of the delegate handles that we have bound to IConsoleVariable::OnChangedDelegate and need to be removed
		 * before it is safe to destroy the manager. Most likely due to having bound a lambda to the delegate that captured the
		 * FVirtualizationManager this pointer.
		 */
		TArray<TPair<IConsoleVariable*, FDelegateHandle>> ConsoleDelegateHandles;

		/** The critical section used to force single threaded access if bForceSingleThreaded is true */
		FCriticalSection ForceSingleThreadedCS;

		/** When enabled all public operations will be performed as single threaded */
		bool bSingleThreaded = false;

		/** 
		 * When enabled we will immediately 'pull' each payload after it has been pushed to either local or persistent
		 * storage and compare it to the original payload source to make sure that it was uploaded correctly 
		 */
		bool bValidateAfterPush = false;

		/** Array of backend names that should have their pull operation disabled */
		TArray<FString> MissBackends;

		/** The chance that a payload pull can just 'fail' to allow for testing */
		float MissChance;

		/** The number of upcoming payload pulls that should be failed */
		std::atomic<int32> MissCount = 0;
	} DebugValues;

public:
	/**
	 * Temp accessor for backends until we add a better system for connection failure notification. Note that
	 * this is not exposed outside of the module so that we can change/remove this easily in the future.
	 */
	static FString GetConnectionHelpUrl();
};

} // namespace UE::Virtualization
