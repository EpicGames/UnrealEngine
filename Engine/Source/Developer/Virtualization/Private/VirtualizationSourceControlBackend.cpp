// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualizationSourceControlBackend.h"

#include "AnalyticsEventAttribute.h"
#include "Containers/Ticker.h"
#include "HAL/Event.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "IO/IoHash.h"
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "Interfaces/IPluginManager.h"
#include "Logging/MessageLog.h"
#include "Logging/StructuredLog.h"
#include "Misc/CommandLine.h"
#include "Misc/FileHelper.h"
#include "Misc/Parse.h"
#include "Misc/PathViews.h"
#include "Misc/Paths.h"
#include "Misc/ScopeExit.h"
#include "Misc/ScopedSlowTask.h"
#include "SVirtualizationRevisionControlConnectionDialog.h"
#include "SourceControlHelpers.h"
#include "SourceControlInitSettings.h"
#include "SourceControlOperations.h"
#include "Virtualization/VirtualizationSystem.h"
#include "VirtualizationManager.h"
#include "VirtualizationSourceControlUtilities.h"
#include "VirtualizationUtilities.h"

// When the SourceControl module (or at least the perforce source control module) is thread safe we
// can enable this and stop using the hacky work around 'TryToDownloadFileFromBackgroundThread'
#define IS_SOURCE_CONTROL_THREAD_SAFE 0

#define LOCTEXT_NAMESPACE "Virtualization"

namespace UE::Virtualization
{

/** 
 * A quick and dirty implementation of a std::counting_semaphore that we can use to 
 * limit the number of threads that can create a new perforce connection when pulling or 
 * pushing payloads.
 * 
 * In the worst case scenario where a user needs to pull all of their payloads from the source
 * control backend rather than a faster backend we need to make sure that they will not overwhelm
 * their server with requests.
 * In the future we can use this sort of limit to help gather requests from many threads into a 
 * single batch request from the server which will be much more efficient than the current 'one
 * payload, one request' system. Although we might want to consider gathering multiple requests
 * at a higher level so that all backends can work on the same batching principle.
 */
class FSemaphore 
{
public:
	UE_NONCOPYABLE(FSemaphore);

	enum class EAcquireResult
	{
		/** The acquire was a success and the thread can continue */
		Success,
		/** The wait event failed, the semaphore object is no longer safe */
		EventFailed
	};

	enum class EFlags : uint32
	{
		None					= 0,
		PrioritizeGameThread	= 1 << 0
	};

	FRIEND_ENUM_CLASS_FLAGS(EFlags);

	FSemaphore() = delete;
	explicit FSemaphore(int32 InitialCount, EFlags Options)
		: WaitEvent(EEventMode::ManualReset)
		, Counter(InitialCount)
		, DebugCount(0)
	{
		bPrioritizeGameThread = EnumHasAnyFlags(Options, EFlags::PrioritizeGameThread);
	}

	~FSemaphore()
	{
		checkf(DebugCount == 0, TEXT("'%d' threads are still waiting on the UE::Virtualization::FSemaphore being destroyed"), DebugCount.load());
	}

	/** Will block until the calling thread can pass through the semaphore. Note that it might return an error if the WaitEvent fails */
	EAcquireResult Acquire()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSemaphore::Acquire);

		CriticalSection.Lock();
		DebugCount++;

		if (bPrioritizeGameThread && IsInGameThread())
		{
			return AcquireFromGameThread();
		}

		while (Counter-- <= 0)
		{
			Counter++;
			WaitEvent->Reset();

			CriticalSection.Unlock();

			if (!WaitEvent->Wait())
			{
				--DebugCount;
				return EAcquireResult::EventFailed;
			}
			CriticalSection.Lock();
		}

		CriticalSection.Unlock();

		return EAcquireResult::Success;
	}

	void Release()
	{
		FScopeLock _(&CriticalSection);

		Counter++;
		WaitEvent->Trigger();

		--DebugCount;
	}

private:

	inline EAcquireResult AcquireFromGameThread()
	{
		if (Counter-- > 0)
		{
			CriticalSection.Unlock();
		}
		else
		{
			WaitEvent->Reset();

			CriticalSection.Unlock();

			if (!WaitEvent->Wait())
			{
				--DebugCount;
				return EAcquireResult::EventFailed;
			}
		}

		return EAcquireResult::Success;
	}

	FEventRef WaitEvent;
	FCriticalSection CriticalSection;

	std::atomic<int32> Counter;
	std::atomic<int32> DebugCount;

	bool bPrioritizeGameThread;
};

ENUM_CLASS_FLAGS(FSemaphore::EFlags);

/** Structure to make it easy to acquire/release a FSemaphore for a given scope */
struct FSemaphoreScopeLock
{
	FSemaphoreScopeLock(FSemaphore* InSemaphore)
		: Semaphore(InSemaphore)
	{
		if (Semaphore != nullptr)
		{
			Semaphore->Acquire();
		}
	}

	~FSemaphoreScopeLock()
	{
		if (Semaphore != nullptr)
		{
			Semaphore->Release();
		}
	}

private:
	FSemaphore* Semaphore;
};

/**
 * This utility can be used to try and find the P4PORT (server address) that the provider 
 * is using. The source control api doesn't actually have a way to do this at the moment, 
 * but we can get a general status text of the connection from which we can attempt to 
 * parse the port.
 */
[[nodiscard]] FString GetPortFromProvider(ISourceControlProvider* SCCProvider)
{
	if (SCCProvider != nullptr)
	{
		const FString Status = SCCProvider->GetStatusText().ToString();

		FString Port;
		if (FParse::Value(*Status, TEXT("Port:"), Port))
		{
			return Port;
		}		
	}

	return FString(TEXT("Unknown"));
}

/** Utility function to create a directory to submit payloads from and add the dummy P4IGNORE file */
[[nodiscard]] static bool TrySetupSubmissionSessionDirectory(FStringView SessionDirectoryPath, FStringView IgnoreFileName)
{
	if (!FPlatformFileManager::Get().GetPlatformFile().CreateDirectoryTree(WriteToString<512>(SessionDirectoryPath).ToString()))
	{
		TStringBuilder<MAX_SPRINTF> SystemErrorMsg;
		Utils::GetFormattedSystemError(SystemErrorMsg);

		UE_LOGFMT(LogVirtualization, Error, "Failed to create '{SubmissionDir}' due to: {SysError}", SessionDirectoryPath, SystemErrorMsg);
		return false;
	}

	// Write out an ignore file to the submission directory (will create the directory if needed)
	{
		TStringBuilder<260> IgnoreFilePath;
		FPathViews::Append(IgnoreFilePath, SessionDirectoryPath, IgnoreFileName);

		// A very basic .p4ignore file that should make sure that we are only submitting valid .upayload files.
		// 
		// Since the file should only exist while we are pushing payloads, it is not expected that anyone will need
		// to read the file. Due to this we only include the bare essentials in terms of documentation.

		TStringBuilder<512> FileContents;

		FileContents << TEXT("# Ignore all files\n*\n\n");
		FileContents << TEXT("# Allow.payload files as long as they are the expected 3 directories deep\n!*/*/*/*.upayload\n\n");

		if (!FFileHelper::SaveStringToFile(FileContents, IgnoreFilePath.ToString()))
		{
			TStringBuilder<MAX_SPRINTF> SystemErrorMsg;
			Utils::GetFormattedSystemError(SystemErrorMsg);

			UE_LOGFMT(LogVirtualization, Error, "Failed to create '{IgnoreFile}' due to: {SysError}", IgnoreFilePath, SystemErrorMsg);
			return false;
		}
	}

	return true;
}

/** Builds a changelist description to be used when submitting a payload to source control */
static void CreateDescription(const FString& ProjectName, TArrayView<const FPushRequest*>& FileRequests, TStringBuilder<512>& OutDescription)
{
	// TODO: Maybe make writing out the project name an option or allow for a codename to be set via ini file?
	OutDescription << TEXT("Submitted for project: ");
	OutDescription << ProjectName;

	bool bInitialNewline = false;

	for (const FPushRequest* Request : FileRequests)
	{
		if (!Request->GetContext().IsEmpty())
		{
			if (!bInitialNewline)
			{
				OutDescription << TEXT("\n");
				bInitialNewline = true;
			}

			OutDescription << TEXT("\n") << Request->GetIdentifier() << "\t: " << Request->GetContext();
		}
	}
}

[[nodiscard]] static ECommandResult::Type GetDepotPathStates(ISourceControlProvider& SCCProvider, const TArray<FString>& DepotPaths, TArray<FSourceControlStateRef>& OutStates)
{
	TSharedRef<FUpdateStatus, ESPMode::ThreadSafe> UpdateOperation = ISourceControlOperation::Create<FUpdateStatus>();
	UpdateOperation->SetRequireDirPathEndWithSeparator(true);

	ECommandResult::Type Result = SCCProvider.Execute(UpdateOperation, DepotPaths);
	if (Result != ECommandResult::Succeeded)
	{
		return Result;
	}

	return SCCProvider.GetState(DepotPaths, OutStates, EStateCacheUsage::Use);
}

/** 
 * If the only reason we failed an operation is because of missing depot files then there
 * is no point retrying that operation however other kinds of errors, such as connection
 * problems, might be not be encountered if we try again.
 * 
 * @param ResultInfo	The results of a failed operation
 * @return				True if there were no errors (as we don't really know what went wrong)
 *						or if there errors not relating to missing files, otherwise false.
 */
[[nodiscard]] static bool ShouldRetryOperation(const FSourceControlResultInfo& ResultInfo)
{
	if (ResultInfo.ErrorMessages.IsEmpty())
	{
		return true;
	}

	// Ideally we'd parse for this sort of thing in the source control module itself and return an error enum
	for (const FText& ErrorTest : ResultInfo.ErrorMessages)
	{
		if (ErrorTest.ToString().Find(" - no such file(s).") == INDEX_NONE)
		{
			return true;
		}
	}

	return false;
}

FSourceControlBackend::FSourceControlBackend(FStringView ProjectName, FStringView ConfigName, FStringView InDebugName)
	: IVirtualizationBackend(ConfigName, InDebugName, EOperations::Push | EOperations::Pull)
	, ProjectName(ProjectName)
{
}

bool FSourceControlBackend::Initialize(const FString& ConfigEntry)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSourceControlBackend::Initialize);

	if (!TryApplySettingsFromConfigFiles(ConfigEntry))
	{
		return false;
	}

	return true;
}

IVirtualizationBackend::EConnectionStatus FSourceControlBackend::OnConnect()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSourceControlBackend::OnConnect);

	FScopedSlowTask SlowTask(1.0f, LOCTEXT("OnConnect", "Connecting to virtualized asset revision control backend..."));
	SlowTask.EnterProgressFrame(1.0f);
	SlowTask.ForceRefresh();

	// First check that the 'PerforceSourceControl' plugin exists and is enabled so that we can give specific
	// error messages if not.
	// We could try to force enable the plugin at this point, in the same way that the stand alone tool does
	// but it is probably better to inform the user and have them add/enable the plugin for their target explicitly
	// rather than have us do it behind the scenes.
	// This is only expected to be a problem when first enabling VA for a project/target and not something a user
	// will experience day to day.
	TSharedPtr<IPlugin> P4Plugin = IPluginManager::Get().FindPlugin("PerforceSourceControl");
	if (!P4Plugin)
	{
		UE_LOGF(LogVirtualization, Error, "[%ls] Failed to find the 'PerforceSourceControl' plugin", *GetDebugName());
		return IVirtualizationBackend::EConnectionStatus::Error;
	}

	if (!P4Plugin->IsEnabled())
	{
		UE_LOGF(LogVirtualization, Error, "[%ls] The 'PerforceSourceControl' plugin is disabled for this target", *GetDebugName());
		return IVirtualizationBackend::EConnectionStatus::Error;
	}

	FString Port = ServerAddress;
	FString UserName;

	bool bSaveSettings = false; // Initially we will try to reason the perforce settings from the ini file and global environment
								// so we don't want to save these values to the users ini files.

	int32 Attempts = 0;
	double TotalConnectionTime = 0.0;

	while (true)
	{
		const double ConnectionStartTime = FPlatformTime::Seconds();

		// TODO: At the moment we cannot get back error messages from ISourceControlProvider::Init, nor can we get back info about
		// the actual login details used, so for the initial dialog we will just have to go with the settings we know and not 
		// give the user any details about the connection problem.

		FText ErrorMessage;
		IVirtualizationBackend::EConnectionStatus ConnectionResult = OnConnectInternal(Port, UserName, bSaveSettings, ErrorMessage);

		// Record each attempt and keep track of the total time we've spent connecting across all attempts.
		Attempts++;
		TotalConnectionTime += FPlatformTime::Seconds() - ConnectionStartTime;

		if (ConnectionResult == IVirtualizationBackend::EConnectionStatus::Connected)
		{
			if (bSaveSettings)
			{
				check(::IsInGameThread());
				GConfig->Flush(false, SourceControlHelpers::GetSettingsIni());
			}

			{
				TArray<FAnalyticsEventAttribute> Attributes;
				Attributes.Reserve(4);

				Attributes.Add({ TEXT("Connected"), true }); // Hard coded for now, but this will make more sense if we combine this event with the failures below
				Attributes.Add({ TEXT("Attempts"), Attempts });
				Attributes.Add({ TEXT("TotalConnectionTime"), TotalConnectionTime });
				Attributes.Add({ TEXT("Interactive"), Utils::IsProcessInteractive() });

				GetAnalyticsRecordEvent().Broadcast(TEXT("Editor.VA.SourceControlOnConnect"), Attributes, EAnalyticsFlags::None);
			}

			return IVirtualizationBackend::EConnectionStatus::Connected;
		}

#if UE_VA_WITH_SLATE
		// If the local ini settings are ignored then there is no point saving correct settings given by the user.
		// They will need to fix their root problem instead.
		// TODO: Maybe give a bespoke error at this point?
		if (bUseRetryConnectionDialog && bUseLocalIniFileSettings && Utils::IsProcessInteractive())
		{
			SRevisionControlConnectionDialog::FResult DialogResult = SRevisionControlConnectionDialog::RunDialog(	TEXT("Perforce"),
																													TEXT("PerforceSourceControl.VirtualizationSettings"),
																													Port,
																													UserName,
																													ErrorMessage);

			{
				TArray<FAnalyticsEventAttribute> Attributes;
				Attributes.Add({ TEXT("UserSelection"), DialogResult.bShouldRetry ? TEXT("Retry") : TEXT("Skip") });

				// Need to flush if we are going to quit to make sure that the analytics payloads are sent properly.
				const EAnalyticsFlags Flags = DialogResult.bShouldRetry ? EAnalyticsFlags::None : EAnalyticsFlags::Flush;

				GetAnalyticsRecordEvent().Broadcast(TEXT("Editor.VA.SourceControlLoginFail"), Attributes, EAnalyticsFlags::None);
			}
			
			if (!DialogResult.bShouldRetry)
			{
				// User has opted to continue without a valid source control connection
				OnConnectionError(ErrorMessage);
				return IVirtualizationBackend::EConnectionStatus::Error;
			}

			Port = DialogResult.Port;
			UserName = DialogResult.UserName;
			bSaveSettings = true; // Now we have input from the user, so we should save it for future sessions
		}
		else
#endif
		{
			OnConnectionError(ErrorMessage);
			return IVirtualizationBackend::EConnectionStatus::Error;
		}
	}
}


IVirtualizationBackend::EConnectionStatus FSourceControlBackend::OnConnectInternal(FString& InOutPort, FString& InOutUsername, bool bSaveConnectionSettings, FText& OutErrorMessage)
{
	// We do not want the connection to have a client workspace so explicitly set it to empty
	FSourceControlInitSettings SCCSettings(FSourceControlInitSettings::EBehavior::OverrideExisting);

	FSourceControlInitSettings::EConfigBehavior IniBehavior = bSaveConnectionSettings	? FSourceControlInitSettings::EConfigBehavior::ReadWrite 
																						: FSourceControlInitSettings::EConfigBehavior::ReadOnly;

	if (!bUseLocalIniFileSettings)
	{
		IniBehavior = FSourceControlInitSettings::EConfigBehavior::None;
	}

	SCCSettings.SetConfigBehavior(IniBehavior);

	if (!InOutPort.IsEmpty())
	{
		SCCSettings.AddSetting(TEXT("P4Port"), InOutPort);
	}

	if (!InOutUsername.IsEmpty())
	{
		SCCSettings.AddSetting(TEXT("P4User"), InOutUsername);
	}

	// By setting an empty "P4Client" we ensure that the provider will not attempt to find a default one.
	SCCSettings.AddSetting(TEXT("P4Client"), TEXT(""));

	SCCProvider = ISourceControlModule::Get().CreateProvider(FName("Perforce"), TEXT("Virtualization"), SCCSettings);
	if (!SCCProvider.IsValid())
	{
		UE_LOGF(LogVirtualization, Error, "[%ls] Failed to create a perforce revision control provider", *GetDebugName());
		return IVirtualizationBackend::EConnectionStatus::Error;
	}

	const ISourceControlProvider::EInitFlags InitOptions =	ISourceControlProvider::EInitFlags::AttemptConnection |
															ISourceControlProvider::EInitFlags::SuppressErrorLogging;

	ISourceControlProvider::FInitResult Result = SCCProvider->Init(InitOptions);

	if (FString* Value = Result.ConnectionSettings.Find(ISourceControlProvider::EStatus::Port))
	{
		InOutPort = *Value;
	}

	if (FString* Value = Result.ConnectionSettings.Find(ISourceControlProvider::EStatus::User))
	{
		InOutUsername = *Value;
	}

	if (!Result.bIsAvailable)
	{
		FTextBuilder MsgBuilder;
		if (!Result.Errors.ErrorMessage.IsEmpty())
		{
			MsgBuilder.AppendLine(Result.Errors.ErrorMessage);
		}

		for (const FText& Error : Result.Errors.AdditionalErrors)
		{
			MsgBuilder.AppendLine(Error);
		}

		OutErrorMessage = MsgBuilder.ToText();
		
		return IVirtualizationBackend::EConnectionStatus::Error;
	}

	// When a source control depot is set up a file named 'payload_metainfo.txt' should be submitted to it's root.
	// This allows us to check for the existence of the file to confirm that the depot root is indeed valid.
	const FString PayloadMetaInfoPath = WriteToString<512>(DepotPathRoot, TEXT("payload_metainfo.txt")).ToString();

	FSharedBuffer MetaInfoBuffer;

#if IS_SOURCE_CONTROL_THREAD_SAFE
	TSharedRef<FDownloadFile, ESPMode::ThreadSafe> DownloadCommand = ISourceControlOperation::Create<FDownloadFile>();
	if (SCCProvider->Execute(DownloadCommand, PayloadMetaInfoPath, EConcurrency::Synchronous) != ECommandResult::Succeeded)
	{
#else
	TSharedRef<FDownloadFile, ESPMode::ThreadSafe> DownloadCommand = ISourceControlOperation::Create<FDownloadFile>();
	if (SCCProvider->TryToDownloadFileFromBackgroundThread(DownloadCommand, PayloadMetaInfoPath))
	{
#endif //IS_SOURCE_CONTROL_THREAD_SAFE

		MetaInfoBuffer = DownloadCommand->GetFileData(PayloadMetaInfoPath);
	}

	if (MetaInfoBuffer.IsNull())
	{
		OutErrorMessage = FText::Format(LOCTEXT("FailedMetaInfo", "Failed to find '{0}' on server '{1}'\nThe revision control backend will be unable to pull payloads, is your revision control config set up correctly?"),
			FText::FromString(PayloadMetaInfoPath),
			FText::FromString(GetPortFromProvider(SCCProvider.Get())));

		return IVirtualizationBackend::EConnectionStatus::Error;
	}

	// Currently we do not do anything with the payload meta info, in the future we could structure
	// it's format to include more information that might be worth logging or something. 
	// But for now being able to pull the payload meta info path at least shows that we can use the
	// depot.
	return IVirtualizationBackend::EConnectionStatus::Connected;
}

bool FSourceControlBackend::PullData(TArrayView<FPullRequest> Requests, EPullFlags Flags, FText& OutErrors)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSourceControlBackend::PullData);

	TArray<FString> DepotPaths;
	DepotPaths.Reserve(Requests.Num());

	for (FPullRequest& Request : Requests)
	{
		TStringBuilder<512> DepotPath;
		CreateDepotPath(Request.GetIdentifier(), DepotPath);

		DepotPaths.Add(DepotPath.ToString());
	}

	// TODO: When multiple threads are blocked waiting on this we could gather X payloads together and make a single
	// batch request on the same connection, which should be a lot faster with less overhead.
	// Although ideally this backend will not get hit very often.
	FSemaphoreScopeLock _(ConcurrentConnectionLimit.Get());

	int32 Retries = 0;

	bool bConnectionFailed = false;

	// Note that 'RetryCount' is badly named, it's actually the number of attempts we can make, not 1 attempt + retries

	if (RetryCount <= 0)
	{
		UE_LOGF(LogVirtualization, Warning, "[%ls] Retry count is set to 0, no attempts can be made!", *GetDebugName());
		return false;
	}

	while (true)
	{
		// Only warn if the backend is configured to retry
		if (Retries != 0)
		{
			UE_LOGF(LogVirtualization, Warning, "[%ls] Failed to pull payloads! Attempt (%d/%d) retrying in %d(ms)...", *GetDebugName(), Retries, RetryCount, RetryWaitTimeMS);
			FPlatformProcess::SleepNoStats(static_cast<float>(RetryWaitTimeMS) * 0.001f);
		}

		TSharedRef<FDownloadFile> DownloadCommand = ISourceControlOperation::Create<FDownloadFile>();
		DownloadCommand->SetEnableErrorLogging(false);

#if IS_SOURCE_CONTROL_THREAD_SAFE
		const bool bOperationSuccess = SCCProvider->Execute(DownloadCommand, DepotPaths, EConcurrency::Synchronous) == ECommandResult::Succeeded;
#else
		const bool bOperationSuccess = SCCProvider->TryToDownloadFileFromBackgroundThread(DownloadCommand, DepotPaths);
#endif

		// Check and assign what payloads we did find, even if the download command failed we might have been able to download some
		// files and since we found them we might as well return them.
		for (int32 Index = 0; Index < Requests.Num(); ++Index)
		{
			// The payload was created by FCompressedBuffer::Compress so we can return it as a FCompressedBuffer.
			FSharedBuffer Buffer = DownloadCommand->GetFileData(DepotPaths[Index]);
			if (!Buffer.IsNull())
			{
				Requests[Index].SetPayload(FCompressedBuffer::FromCompressed(Buffer));
			}
		}

		if (bOperationSuccess)
		{
			return true;
		}

		// If this was the first try then check to see if the error being returns is that the file does not exist
		// in the depot. If it does not exist then there is no point in us retrying and we can error out at this point.
		if (Retries == 0 && !ShouldRetryOperation(DownloadCommand->GetResultInfo()))
		{
			return false;
		}

		if (++Retries >= RetryCount)
		{
			UE_LOGF(LogVirtualization, Warning, "[%ls] Failed to pull payloads! Attempt (%d/%d)", *GetDebugName(), Retries, RetryCount);
			
			FTextBuilder ErrorBuilder;
			if (DownloadCommand->GetResultInfo().DidConnectionFail())
			{
				TMap<ISourceControlProvider::EStatus, FString> SCPStatus = SCCProvider->GetStatus();

				FText ErrorMsg = FText::Format(LOCTEXT("VA_SCP", "Failed to connect to perforce server '{0}' with username '{1}'"),
					FText::FromString(SCPStatus[ISourceControlProvider::EStatus::Port]),
					FText::FromString(SCPStatus[ISourceControlProvider::EStatus::User]));

				ErrorBuilder.AppendLine(ErrorMsg);
			}

			for (const FText& ErrorMsg : DownloadCommand->GetResultInfo().ErrorMessages)
			{
				ErrorBuilder.AppendLine(ErrorMsg);
			}

			if (!ErrorBuilder.IsEmpty())
			{
				OutErrors = ErrorBuilder.ToText();
			}

			return false;
		}
	}
}

bool FSourceControlBackend::DoesPayloadExist(const FIoHash& Id)
{
	TArray<bool> Result;

	if (FSourceControlBackend::DoPayloadsExist(MakeArrayView<const FIoHash>(&Id, 1), Result))
	{
		check(Result.Num() == 1);
		return Result[0];
	}
	else
	{
		return false;
	}
}

bool FSourceControlBackend::PushData(TArrayView<FPushRequest> Requests, EPushFlags Flags)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSourceControlBackend::PushData);

	// First check to see which payloads actually need pushing, if we are lucky we might
	// not need to even bother with workspace manipulation if all payloads already exist.

	FSemaphoreScopeLock _(ConcurrentConnectionLimit.Get());

	// TODO: At some point FVirtualizationManager will do this so we know there cannot be duplicate
	// payload requests, at which point we can rework this code.
	// We create a new array of requests that we know are unique along with a map of the request 
	// results that we can use to set the original requests statuses at the end.

	TMap<FIoHash, FPushResult> PayloadStatusMap;
	PayloadStatusMap.Reserve(Requests.Num());

	TArray<const FPushRequest*> UniqueRequests;
	UniqueRequests.Reserve(Requests.Num());

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSourceControlBackend::PushData::FindUniquePayloads);

		for (const FPushRequest& Request : Requests)
		{
			if (!PayloadStatusMap.Contains(Request.GetIdentifier()))
			{
				PayloadStatusMap.Add(Request.GetIdentifier(), FPushResult::GetAsError());
				UniqueRequests.Add(&Request);
			}
		}

		UE_LOGF(LogVirtualization, Log, "[%ls] Found %d unique payload(s)", *GetDebugName(), UniqueRequests.Num());
	}

	TArray<const FPushRequest*> RequestsToPush;
	RequestsToPush.Reserve(UniqueRequests.Num());

	// Note that we do not check EPushFlags::Force here and always check if the payloads are already in
	// source control as we do not want multiple revisions.
	// This might change at some point if we want to try changing the compression codec of the stored
	// payloads.

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSourceControlBackend::PushData::CheckIfPayloadsAlreadyExist);

		const int32 NumBatches = FMath::DivideAndRoundUp<int32>(UniqueRequests.Num(), MaxBatchCount);

		TArray<FIoHash> PayloadIdentifiers;
		PayloadIdentifiers.Reserve(MaxBatchCount);

		TArray<bool> PayloadResults;
		PayloadResults.Reserve(MaxBatchCount);

		for (int32 BatchIndex = 0; BatchIndex < NumBatches; ++BatchIndex)
		{
			PayloadIdentifiers.Reset(0);
			PayloadResults.Reset(0);

			const int32 BatchStart = BatchIndex * MaxBatchCount;
			const int32 BatchEnd = FMath::Min((BatchIndex + 1) * MaxBatchCount, UniqueRequests.Num());

			for (int32 ReqIndex = BatchStart; ReqIndex < BatchEnd; ++ReqIndex)
			{
				PayloadIdentifiers.Add(UniqueRequests[ReqIndex]->GetIdentifier());
			}

			if (!DoPayloadsExist(PayloadIdentifiers, PayloadResults))
			{
				UE_LOGF(LogVirtualization, Error, "[%ls] Failed to find the current file state for payloads", *GetDebugName());
				return false;
			}

			check(PayloadIdentifiers.Num() == PayloadResults.Num());

			for (int32 Index = 0; Index < PayloadResults.Num(); ++Index)
			{
				const FPushRequest* Request = UniqueRequests[BatchStart + Index];

				if (PayloadResults[Index])
				{
					PayloadStatusMap[Request->GetIdentifier()] = FPushResult::GetAsAlreadyExists();
				}
				else
				{
					RequestsToPush.Add(Request);
				}
			}
		}
	}

	UE_LOGF(LogVirtualization, Log, "[%ls] Determined that %d payload(s) require submission to revision control", *GetDebugName(), RequestsToPush.Num());

	if (RequestsToPush.IsEmpty())
	{
		for (FPushRequest& Request : Requests)
		{
			Request.SetResult(PayloadStatusMap[Request.GetIdentifier()]);
		}

		return true;
	}

	// TODO: Consider creating one workspace and one temp dir per session rather than per push.
	// Although this would require more checking on start up to check for lingering workspaces
	// and directories in case of editor crashes.
	// We'd also need to remove each submitted file from the workspace after submission so that
	// we can delete the local file

	// We cannot easily submit files from within the project root due to p4 ignore rules
	// so we will use the user temp directory instead. We append a guid to the root directory
	// to avoid potentially conflicting with other editor processes that might be running.

	const FGuid SessionGuid = FGuid::NewGuid();
	
	UE_LOGF(LogVirtualization, Log, "[%ls] Started payload submission session '%ls' for '%d' payload(s)", *GetDebugName(), *LexToString(SessionGuid), RequestsToPush.Num());

	TStringBuilder<260> SessionDirectory;
	FPathViews::Append(SessionDirectory, SubmissionRootDir, SessionGuid);

	if (!TrySetupSubmissionSessionDirectory(SessionDirectory, IgnoreFileName))
	{
		UE_LOGF(LogVirtualization, Error, "[%ls] Failed to setup the directory to submit payloads from", *GetDebugName());
		return false;
	}

	UE_LOGF(LogVirtualization, Log, "[%ls] Created directory '%ls' to submit payloads from", *GetDebugName(), SessionDirectory.ToString());

	ON_SCOPE_EXIT
	{
		// Clean up the payload file from disk and the temp directories, but we do not need to give errors if any of these operations fail.
		IFileManager::Get().DeleteDirectory(SessionDirectory.ToString(), false, true);
	};

	TStringBuilder<64> WorkspaceName;
	WorkspaceName << TEXT("VASubmission-") << SessionGuid;

	// Create a temp workspace so that we can submit the payload from
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSourceControlBackend::PushData::CreateWorkspace);
		TSharedRef<FCreateWorkspace> CreateWorkspaceCommand = ISourceControlOperation::Create<FCreateWorkspace>(WorkspaceName, SessionDirectory);

		if (ClientStream.IsEmpty())
		{
			TStringBuilder<512> DepotMapping;
			DepotMapping << DepotPathRoot << TEXT("...");

			TStringBuilder<128> ClientMapping;
			ClientMapping << TEXT("//") << WorkspaceName << TEXT("/...");

			CreateWorkspaceCommand->AddNativeClientViewMapping(DepotMapping, ClientMapping);
		}
		else
		{
			CreateWorkspaceCommand->SetStream(ClientStream);
		}

		if (bUsePartitionedClient)
		{
			CreateWorkspaceCommand->SetType(FCreateWorkspace::EType::Partitioned);
		}

		CreateWorkspaceCommand->SetDescription(TEXT("This workspace was auto generated when submitting virtualized payloads to revision control"));

		if (SCCProvider->Execute(CreateWorkspaceCommand) != ECommandResult::Succeeded)
		{
			UE_LOGF(LogVirtualization, Error, "[%ls] Failed to create temp workspace '%ls' to use for payload submission",
				*GetDebugName(),
				WorkspaceName.ToString());

			return false;
		}
	}

	ON_SCOPE_EXIT
	{
		// Remove the temp workspace mapping
		if (SCCProvider->Execute(ISourceControlOperation::Create<FDeleteWorkspace>(WorkspaceName)) != ECommandResult::Succeeded)
		{
			UE_LOGF(LogVirtualization, Warning, "[%ls] Failed to remove temp workspace '%ls' please delete manually", *GetDebugName(), WorkspaceName.ToString());
		}
	};

	FString OriginalWorkspace;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSourceControlBackend::PushData::SwitchWorkspace);

		FSourceControlResultInfo SwitchToNewWorkspaceInfo;
		if (SCCProvider->SwitchWorkspace(WorkspaceName, SwitchToNewWorkspaceInfo, &OriginalWorkspace) != ECommandResult::Succeeded)
		{
			UE_LOGF(LogVirtualization, Error, "[%ls] Failed to switch to temp workspace '%ls' when trying to submit payloads",
				*GetDebugName(),
				WorkspaceName.ToString());

			return false;
		}
	}

	ON_SCOPE_EXIT
	{
		FSourceControlResultInfo SwitchToOldWorkspaceInfo;
		if (SCCProvider->SwitchWorkspace(OriginalWorkspace, SwitchToOldWorkspaceInfo, nullptr) != ECommandResult::Succeeded)
		{
			UE_LOGF(LogVirtualization, Error, "[%ls] Failed to restore the original workspace '%ls'", *GetDebugName(), *OriginalWorkspace);
		}
	};

	const int32 NumBatches = FMath::DivideAndRoundUp<int32>(RequestsToPush.Num(), MaxBatchCount);

	UE_LOGF(LogVirtualization, Log, "[%ls] Splitting the submission into '%d' batches", *GetDebugName(), NumBatches);

	for (int32 BatchIndex = 0; BatchIndex < NumBatches; ++BatchIndex)
	{
		UE_LOGF(LogVirtualization, Log, "[%ls] Processing batch %d/%d...", *GetDebugName(), BatchIndex + 1, NumBatches);

		const int32 BatchStart = BatchIndex * MaxBatchCount;
		const int32 BatchEnd = FMath::Min((BatchIndex + 1) * MaxBatchCount, RequestsToPush.Num());

		TArrayView<const FPushRequest*> RequestBatch(&RequestsToPush[BatchStart], BatchEnd - BatchStart);

		TArray<FString> FilesToSubmit;
		FilesToSubmit.Reserve(RequestBatch.Num());

		// Write the payloads to disk so that they can be submitted (source control module currently requires the files to
		// be on disk)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FSourceControlBackend::PushData::CreateFiles);

			for (const FPushRequest* Request : RequestBatch)
			{
				check(Request != nullptr);

				TStringBuilder<52> LocalPayloadPath;
				Utils::PayloadIdToPath(Request->GetIdentifier(), LocalPayloadPath);

				TStringBuilder<260> PayloadFilePath;
				FPathViews::Append(PayloadFilePath, SessionDirectory, LocalPayloadPath);

				UE_LOGF(LogVirtualization, Verbose, "[%ls] Writing payload to '%ls' for submission", *GetDebugName(), PayloadFilePath.ToString());

				FCompressedBuffer Payload = Request->GetPayload();
				if (Payload.IsNull())
				{
					UE_LOGF(LogVirtualization, Error, "[%ls] Failed to acquire payload '%ls' contents to '%ls' for writing",
						*GetDebugName(),
						*LexToString(Request->GetIdentifier()),
						PayloadFilePath.ToString());

					return false;
				}

				TUniquePtr<FArchive> FileAr(IFileManager::Get().CreateFileWriter(PayloadFilePath.ToString()));
				if (!FileAr)
				{
					TStringBuilder<MAX_SPRINTF> SystemErrorMsg;
					Utils::GetFormattedSystemError(SystemErrorMsg);

					UE_LOGF(LogVirtualization, Error, "[%ls] Failed to write payload '%ls' contents to '%ls' due to system error: %ls",
						*GetDebugName(),
						*LexToString(Request->GetIdentifier()),
						PayloadFilePath.ToString(),
						SystemErrorMsg.ToString());

					return false;
				}

				Payload.Save(*FileAr);

				if (!FileAr->Close())
				{
					TStringBuilder<MAX_SPRINTF> SystemErrorMsg;
					Utils::GetFormattedSystemError(SystemErrorMsg);

					UE_LOGF(LogVirtualization, Error, "[%ls] Failed to write payload '%ls' contents to '%ls' due to system error: %ls",
						*GetDebugName(),
						*LexToString(Request->GetIdentifier()),
						*PayloadFilePath,
						SystemErrorMsg.
						ToString());

					return false;
				}

				FilesToSubmit.Emplace(MoveTemp(PayloadFilePath));
			}
		}

		check(RequestBatch.Num() == FilesToSubmit.Num());

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FSourceControlBackend::PushData::AddFiles);

			if (SCCProvider->Execute(ISourceControlOperation::Create<FMarkForAdd>(), FilesToSubmit) != ECommandResult::Succeeded)
			{
				UE_LOGF(LogVirtualization, Error, "[%ls] Failed to mark the payload file for Add in revision control", *GetDebugName());
				return false;
			}
		}

		// Now submit the payload
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FSourceControlBackend::PushData::SubmitFiles);

			TSharedRef<FCheckIn, ESPMode::ThreadSafe> CheckInOperation = ISourceControlOperation::Create<FCheckIn>();

			TStringBuilder<512> Description;
			CreateDescription(ProjectName, RequestBatch, Description);

			CheckInOperation->SetDescription(FText::FromString(Description.ToString()));

			if (SCCProvider->Execute(CheckInOperation, FilesToSubmit) != ECommandResult::Succeeded)
			{
				UE_LOGF(LogVirtualization, Error, "[%ls] Failed to submit the payload file(s) to revision control", *GetDebugName());
				return false;
			}
		}

		for (const FPushRequest* Request : RequestBatch)
		{
			PayloadStatusMap[Request->GetIdentifier()] = FPushResult::GetAsPushed();
		}

		// Try to clean up the files from this batch
		for (const FString& FilePath : FilesToSubmit)
		{
			const bool bRequireExists = false;
			const bool bEvenReadOnly = true;
			const bool bQuiet = false;

			IFileManager::Get().Delete(*FilePath, bRequireExists, bEvenReadOnly, bQuiet);
		}
	}

	// Finally set all of the request statuses
	for (FPushRequest& Request : Requests)
	{
		Request.SetResult(PayloadStatusMap[Request.GetIdentifier()]);
	}

	return true;
}

bool FSourceControlBackend::DoPayloadsExist(TArrayView<const FIoHash> PayloadIds, TArray<bool>& OutResults)
{
	TArray<FString> DepotPaths;
	DepotPaths.Reserve(PayloadIds.Num());

	TArray<FSourceControlStateRef> PathStates;

	for (const FIoHash& PayloadId : PayloadIds)
	{
		if (!PayloadId.IsZero())
		{
			TStringBuilder<52> LocalPayloadPath;
			Utils::PayloadIdToPath(PayloadId, LocalPayloadPath);

			DepotPaths.Emplace(WriteToString<512>(DepotPathRoot, LocalPayloadPath));
		}
	}

	ECommandResult::Type Result = GetDepotPathStates(*SCCProvider, DepotPaths, PathStates);
	if (Result != ECommandResult::Type::Succeeded)
	{
		UE_LOGF(LogVirtualization, Error, "[%ls] Failed to query the state of files in the revision control depot", *GetDebugName());
		return false;
	}

	check(DepotPaths.Num() == PathStates.Num()); // We expect that all paths return a state

	OutResults.SetNum(PayloadIds.Num());

	int32 StatusIndex = 0;
	for (int32 Index = 0; Index < PayloadIds.Num(); ++Index)
	{
		if (!PayloadIds[Index].IsZero())
		{
			OutResults[Index] = PathStates[StatusIndex++]->IsSourceControlled();
		}
	}

	return true;
}

bool FSourceControlBackend::TryApplySettingsFromConfigFiles(const FString& ConfigEntry)
{
	// If the depot root is within a perforce stream then we must specify which stream. This may be a virtual stream with a custom view.
	{
		FParse::Value(*ConfigEntry, TEXT("ClientStream="), ClientStream);
		if (!ClientStream.IsEmpty())
		{
			UE_LOGF(LogVirtualization, Log, "[%ls] Using client stream: '%ls'", *GetDebugName(), *ClientStream);
		}
		else
		{
			UE_LOGF(LogVirtualization, Log, "[%ls] Not using client stream", *GetDebugName());
		}
	}

	// We require that a valid depot root has been provided
	{
		if (FParse::Value(*ConfigEntry, TEXT("DepotRoot="), DepotPathRoot))
		{
			UE_LOGF(LogVirtualization, Warning, "[%ls] Entry 'DepotRoot' is deprecated, replace with 'DepotPath'", *GetDebugName());
		}
		else
		{
			FParse::Value(*ConfigEntry, TEXT("DepotPath="), DepotPathRoot);
		}

		if (DepotPathRoot.IsEmpty())
		{
			DepotPathRoot = ClientStream;
		}

		if (DepotPathRoot.IsEmpty())
		{
			UE_LOGF(LogVirtualization, Error, "[%ls] 'DepotPath' was not found in the config file!", *GetDebugName());
			return false;
		}

		if (!DepotPathRoot.EndsWith(TEXT("/")))
		{
			DepotPathRoot.AppendChar(TEXT('/'));
		}
	}

	{
		FParse::Value(*ConfigEntry, TEXT("Server="), ServerAddress);

		if (!ServerAddress.IsEmpty())
		{
			UE_LOGF(LogVirtualization, Log, "[%ls] Using server address: '%ls'", *GetDebugName(), *ServerAddress);
		}
		else
		{
			UE_LOGF(LogVirtualization, Log, "[%ls] Will connect to the default server address", *GetDebugName());
		}
	}

	// Check to see if we should use partitioned clients or not. This is a perforce specific optimization to make the workspace churn cheaper on the server
	{
		FParse::Bool(*ConfigEntry, TEXT("UsePartitionedClient="), bUsePartitionedClient);
		UE_LOGF(LogVirtualization, Log, "[%ls] Using partitioned clients: '%ls'", *GetDebugName(), bUsePartitionedClient ? TEXT("true") : TEXT("false"));
	}

	// Allow the source control backend to retry failed pulls
	{
		int32 RetryCountIniFile = INDEX_NONE;
		if (FParse::Value(*ConfigEntry, TEXT("RetryCount="), RetryCountIniFile))
		{
			RetryCount = RetryCountIniFile;
		}

		int32 RetryWaitTimeMSIniFile = INDEX_NONE;
		if (FParse::Value(*ConfigEntry, TEXT("RetryWaitTime="), RetryWaitTimeMSIniFile))
		{
			RetryWaitTimeMS = RetryWaitTimeMSIniFile;
		}

		UE_LOGF(LogVirtualization, Log, "[%ls] Will retry failed downloads attempts %d time(s) with a gap of %dms between them", *GetDebugName(), RetryCount, RetryWaitTimeMS);
	}

	// Allow the number of concurrent connections to be limited
	{
		int32 MaxLimit = 8; // We use the UGS max of 8 as the default
		FParse::Value(*ConfigEntry, TEXT("MaxConnections="), MaxLimit);

		if (MaxLimit != INDEX_NONE)
		{
			ConcurrentConnectionLimit = MakeUnique<FSemaphore>(MaxLimit, FSemaphore::EFlags::PrioritizeGameThread);
			UE_LOGF(LogVirtualization, Log, "[%ls] Limited to %d concurrent revision control connections", *GetDebugName(), MaxLimit);
		}
		else
		{
			UE_LOGF(LogVirtualization, Log, "[%ls] Has no limit to it's concurrent revision control connections", *GetDebugName());
		}
	}

	// Check for the optional BatchCount parameter
	{
		int32 MaxBatchCountIniFile = 0;
		if (FParse::Value(*ConfigEntry, TEXT("MaxBatchCount="), MaxBatchCountIniFile))
		{
			MaxBatchCount = MaxBatchCountIniFile;
		}

		UE_LOGF(LogVirtualization, Log, "[%ls] Will push payloads in batches of up to %d payloads(s) at a time", *GetDebugName(), MaxBatchCount);
	}

	// Check to see if connection error notification pop ups should be shown or not
	{
		FParse::Bool(*ConfigEntry, TEXT("SuppressNotifications="), bSuppressNotifications);

		if (bSuppressNotifications)
		{
			UE_LOGF(LogVirtualization, Log, "[%ls] Connection pop up warnings are suppressed", *GetDebugName());
		}
		else
		{
			UE_LOGF(LogVirtualization, Log, "[%ls] Connection pop up warnings will be shown", *GetDebugName());
		}
	}

	{
		FParse::Bool(*ConfigEntry, TEXT("UseLocalIniFileSettings="), bUseLocalIniFileSettings);
		UE_LOGF(LogVirtualization, Log, "[%ls] Reading settings from local SourceControlSettings.ini is %ls", *GetDebugName(), bUseLocalIniFileSettings ? TEXT("enabled") : TEXT("disabled"));	
	}

	{
		FParse::Bool(*ConfigEntry, TEXT("UseRetryConnectionDialog="), bUseRetryConnectionDialog);

		if (FParse::Param(FCommandLine::Get(), TEXT("VADisableDialog")))
		{
			bUseRetryConnectionDialog = false;
		}

		UE_LOGF(LogVirtualization, Log, "[%ls] Showing a reconnect dialog on initial failure %ls", *GetDebugName(), bUseRetryConnectionDialog ? TEXT("enabled") : TEXT("disabled"));
	}

	{
		// TODO: We should just extract this from the perforce environment but that requires extending
		// the source control api.
		// Letting the backend define the ignore filename to use is a quicker work around
		if (FParse::Value(*ConfigEntry, TEXT("IgnoreFile="), IgnoreFileName))
		{
#if 0
			if (IgnoreFileName.Contains(TEXT("\\")) || IgnoreFileName.Contains(TEXT("/")))
			{
				UE_LOGF(LogVirtualization, Error, "[%ls] Invalid IgnoreFile entry! This value should just be the filename and not contain any path info", *GetDebugName());
				return false;
			}
#endif
		}

		UE_LOGF(LogVirtualization, Log, "[%ls] Using '%ls' as the p4 ignore file name", *GetDebugName(), *IgnoreFileName);
	}

	if (!FindSubmissionWorkingDir(ConfigEntry))
	{
		return false;
	}

	return true;
}

void FSourceControlBackend::CreateDepotPath(const FIoHash& PayloadId, FStringBuilderBase& OutPath)
{
	TStringBuilder<52> PayloadPath;
	Utils::PayloadIdToPath(PayloadId, PayloadPath);

	OutPath << DepotPathRoot << PayloadPath;
}

bool FSourceControlBackend::FindSubmissionWorkingDir(const FString& ConfigEntry)
{
	// Note regarding path lengths.
	// During submission each payload path will be 90 characters in length which will then be appended to
	// the SubmissionWorkingDir

	FString WorkingDirFromIniFile;
	if (FParse::Value(*ConfigEntry, TEXT("WorkingDir="), WorkingDirFromIniFile) && !WorkingDirFromIniFile.IsEmpty())
	{
		TStringBuilder<512> ExpandedPath;
		if (Utils::ExpandEnvironmentVariables(WorkingDirFromIniFile, ExpandedPath))
		{
			SubmissionRootDir = ExpandedPath;
		}
		else
		{
			UE_LOGF(LogVirtualization, Log, "[%ls] Failed to correctly expand 'WorkingDir=%ls'", *GetDebugName(), *WorkingDirFromIniFile);
		}
	}
	
	if (SubmissionRootDir.IsEmpty())
	{
		SubmissionRootDir = FPlatformMisc::GetEnvironmentVariable(TEXT("UE-VirtualizationWorkingDir"));
		if (!SubmissionRootDir.IsEmpty())
		{
			UE_LOGF(LogVirtualization, Log, "[%ls] Found Environment Variable: UE-VirtualizationWorkingDir", *GetDebugName());
		}
	}

	if (SubmissionRootDir.IsEmpty())
	{

		bool bSubmitFromTempDir = false;
		if (FParse::Bool(*ConfigEntry, TEXT("SubmitFromTempDir="), bSubmitFromTempDir))
		{
			UE_LOGF(LogVirtualization, Warning, "[%ls] Found legacy ini file setting 'SubmitFromTempDir' use '-WorkingDir=$(Temp)/UnrealEngine/VASubmission' instead!", *GetDebugName());
		}

		TStringBuilder<260> PathBuilder;
		if (bSubmitFromTempDir)
		{
			FPathViews::Append(PathBuilder, FPlatformProcess::UserTempDir(), TEXT("UnrealEngine/VASubmission"));	
		}
		else
		{
			FPathViews::Append(PathBuilder, FPaths::ProjectSavedDir(), TEXT("VASubmission"));
		}

		SubmissionRootDir = PathBuilder;
	}

	FPaths::NormalizeDirectoryName(SubmissionRootDir);

	if (IFileManager::Get().DirectoryExists(*SubmissionRootDir) || IFileManager::Get().MakeDirectory(*SubmissionRootDir))
	{
		UE_LOGF(LogVirtualization, Log, "[%ls] Setting '%ls' as the working directory", *GetDebugName(), *SubmissionRootDir);
		return true;
	}
	else
	{
		TStringBuilder<MAX_SPRINTF> SystemErrorMsg;
		Utils::GetFormattedSystemError(SystemErrorMsg);

		UE_LOGF(LogVirtualization, Error, "[%ls] Failed to set the  working directory to '%ls' due to %ls", *GetDebugName(), *SubmissionRootDir, SystemErrorMsg.ToString());
		SubmissionRootDir.Empty();

		return false;
	}
}

void FSourceControlBackend::OnConnectionError(FText ErrorMessage)
{
	// We don't know when or where this error might occur. We can currently only create 
	// FMessageLog on the game thread or risk slate asserts, and if we raise a 
	// FMessageLog::Notify too early in the editor loading process it won't be shown 
	// correctly.
	// To avoid this we push errors to the next editor tick, which will be on the 
	// GameThread and at a point where notification will work. In addition if we do need
	// to defer the error message until next tick (rather than just the notification) then
	// we will write it to the log at the point it is raised rather than the point it is
	// deferred to in an attempt to make the log more readable.

	if (ErrorMessage.IsEmpty())
	{
		return;
	}

	const FText UserMessage(LOCTEXT("FailedSourceControlConnection", "Failed to connect to revision control backend, see the Message Log 'Revision Control' errors for details.\nThe revision control backend will be unable to pull payloads, is your revision control config set up correctly?"));

	TSharedRef<FTokenizedMessage> TokenizedMsg = FTokenizedMessage::Create(EMessageSeverity::Error);
	TokenizedMsg->AddToken(FTextToken::Create(UserMessage));

	FString ConnectionHelpUrl = FVirtualizationManager::GetConnectionHelpUrl();
	if (!ConnectionHelpUrl.IsEmpty())
	{
		TokenizedMsg->AddToken(FURLToken::Create(ConnectionHelpUrl, LOCTEXT("URL", "Additional connection help")));
	}

	if (!Utils::IsProcessInteractive())
	{
		UE_LOGF(LogVirtualization, Warning, "%ls", *ErrorMessage.ToString());
		UE_LOGF(LogVirtualization, Warning, "%ls", *UserMessage.ToString());

		return;
	}
	
	if (::IsInGameThread())
	{
		// If we are on the game thread we can post the error message immediately
		{
			FMessageLog Log("LogVirtualization");
			Log.Error(ErrorMessage);
		}

		{
			FMessageLog Log("LogVirtualization");
			Log.AddMessage(TokenizedMsg);
		}
	}
	else
	{
		// We can only send a FMessageLog on the GameThread so for now just log the error
		// and we can send it to the FMessageLog system next tick
		UE_LOGF(LogVirtualization, Error, "%ls", *ErrorMessage.ToString());
		UE_LOGF(LogVirtualization, Error, "%ls", *UserMessage.ToString());

		auto Callback = [ErrorMessage, TokenizedMsg, bShouldNotify = !bSuppressNotifications](float Delta)->bool
		{
			{
				FMessageLog Log("LogVirtualization");
				Log.SuppressLoggingToOutputLog();
				Log.Error(ErrorMessage);
			}

			{
				FMessageLog Log("LogVirtualization");
				Log.SuppressLoggingToOutputLog();
				Log.AddMessage(TokenizedMsg);
			}

			return false;
		};

		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda(MoveTemp(Callback)));
	}

	if (!bSuppressNotifications)
	{
		// Add the notification to the ticker so that if this is called during editor startup it will only
		// be issued on the first frame, increasing the chance that the user will see it.
		auto NotificationCallback = [](float Delta)->bool
			{
				FMessageLog Notification("LogVirtualization");
				Notification.SuppressLoggingToOutputLog();

				Notification.Notify(LOCTEXT("ConnectionError", "Asset virtualization connection errors were encountered, see the message log for more info"));

				return false;
			};

		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda(MoveTemp(NotificationCallback)));
	}
}

UE_REGISTER_VIRTUALIZATION_BACKEND_FACTORY(FSourceControlBackend, P4SourceControl);
UE_REGISTER_VIRTUALIZATION_BACKEND_FACTORY_LEGACY(FSourceControlBackend, SourceControl, P4SourceControl);

} // namespace UE::Virtualization

#undef LOCTEXT_NAMESPACE
