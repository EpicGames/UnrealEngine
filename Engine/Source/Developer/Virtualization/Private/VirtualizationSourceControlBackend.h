// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "IVirtualizationBackend.h"

#include "Containers/StringView.h"

class ISourceControlProvider;

namespace UE::Virtualization
{

class FSemaphore;

/**
 * This backend can be used to access payloads stored in source control.
 * The backend doesn't 'check out' a payload file but instead will just download the payload as
 * a binary blob.
 * It is assumed that the files are stored with the same path convention as the file system
 * backend, found in Utils::PayloadIdToPath.
 * ----------------------------------------------------------------------------
 * 
 * Ini file setup:
 * 'Name'=(Type=P4SourceControl)
 * Where 'Name' is the backend name in the hierarchy
 * 
 * Required Values:
 * 
 * DepotPath [string]				The path (in depot format) to the location where virtualized payloads are stored.
 *									[Default=""]
 * ClientStream [string]:			Used when the payloads are stored in a stream based depot. It should contain
 *									the stream name to use when creating a workspace for payload submission. [Default=""]
 * 
 * If the payloads are being stored in a depot with type 'stream' them config value 'ClientStream' must be set to a valid
 * stream. The 'DepotPath' value can still be set to choose a specific location within that stream or when the stream 
 * name is not a valid depot path (such as virtual streams)
 * 
 * If the payloads are not being stored in a 'stream' depot then only 'DepotPath' is required.
 * 
 * Optional Values:
 * 
 * Server [string]:					When set the backend will use this server address to connect to. When not
 *									set the backend will use the environment defaults. [Default=""]
 * UsePartitionedClient [bool]:		When true the temporary workspace client created to submit payloads 
 *									from will be created as a partitioned workspace which is less overhead
 *									on the source control server. If your server does not support this then
 *									use false. [Default=True]
 * WorkingDir [string]:				Sets the temp location on disk where payloads are submitted from. The path can
 *									contain config file expansions (see ConfigCacheIni.cpp ::MatchExpansions) and
 *									environment variables with the format $(EnvVarName).
 *									[Default="%GAMESAVEDDIR%/VASubmission"]
 * RetryCount [int32]:				How many times we should try to download a payload before giving up with
 *									an error. Useful when the connection is unreliable but does not experience 
 *									frequent persistent outages. [Default=2]
 * RetryWaitTime [int32]:			The length of time the process should wait between each download attempt
 *									in milliseconds. Remember that the max length of time that the process
 *									can stall attempting to download a payload file is 
 *									RetryCount * RetryWaitTime; [Default=100ms]
 * MaxConnections [int32]			The maximum number of perforce connections that can be made concurrently. 
 *									If this value is exceeded then additional requests will wait until a 
 *									connection becomes free. Pass in '-1' to disable this feature and allow
 *									unlimited connections. [Default=8]
 * BatchCount [int32]				The max number of payloads that can be pushed to source control in a
 *									single submit. If the number of payloads in a request batch exceeds
 *									this size then it will be split into multiple smaller batches. [Default=100]
 * SuppressNotifications[bool]:		When true the system will not display a pop up notification when a 
 *									connection error occurs, allowing the user to stay unaware of the error
 *									unless it actually causes some sort of problem. [Default=false]
 * UseLocalIniFileSettings[bool]	When true the revision control provider will be allowed to load connection 
 *									settings from the users locally saved 'SourceControlSettings.ini' file, if 
 *									false then the settings in this file (if any) will be ignored. [Default=true]
 * IgnoreFile [string]:				Sets the name of the p4 ignore file to use. When submitting payloads we 
 *									create a custom p4 ignore file to override any ignore settings for a project
 *									which allows us to submit from the saved directory which is normally prevented
 *									by the default ignore file. This value should be set to the value of P4IGNORE
									used by your environment. [Default=".p4ignore.txt"]
 * UseRetryConnectionDialog[bool]	When true a slate dialog will be shown if the initial connection to the 
 *									source control server fails allowing the user to attempt to input the correct
 *									login values. [Default=false]
 * 
 * Environment Variables:
 * UE-VirtualizationWorkingDir [string]:	This can be set to a valid directory path that the backend
 *											should use as the root location to submit payloads from.
 *											If the users machine has this set then 'SubmitFromTempDir' 
 *											will be ignored. 
 * ----------------------------------------------------------------------------
 * 
 * Perforce Storage  Setup:
 * 
 * Once you have decided where in perforce you want to store the virtualized payloads you will need to submit
 * a dummy file to the root of that location named "payload_metainfo.txt" preferably with a short message describing
 * the storage locations purpose.
 * When initialized this backend will first attempt to find and download this dummy file as a way of validating that
 * a) the process can correctly connect to perforce 
 * b) the process has the correct location setup in the config file.
 * This validation not only allows us to alert the user to misconfiguration issues before they need to access the data
 * but allows us to prevent users from accidentally pushing virtualized payloads to the wrong location.
 * 
 * So for example, if the virtualized payloads are to be stored under '//payloads/project/...' then the backend would
 * expect to find a '//payloads/project/payload_metainfo.txt' file that it would download via a p4 print command.
 * 
 * If this file is not found then the backend will log an error the first time that it attempts to connect and will prevent
 * the user from pulling or pushing payloads until the issue is resolved.
 * ----------------------------------------------------------------------------
 * 
 * Perforce Settings Setup:
 * 
 * The backend will attempt to use the current global environment settings for perforce to determine 
 * the P4PORT and P4USER values. If you wish to set them via an ini file then they can be done per
 * user by adding the following to Saved\Config\<platform>Editor\SourceControlSettings.ini
 * 
 * -----
 * [PerforceSourceControl.VirtualizationSettings]
 * Port=<server address>
 * UserName=<username>
 * -----
 * Note that the backend will read from these ini file settings but will not write to them.
 */
class FSourceControlBackend final : public IVirtualizationBackend
{
public:
	explicit FSourceControlBackend(FStringView ProjectName, FStringView ConfigName, FStringView InDebugName);
	virtual ~FSourceControlBackend() = default;
	
private:
	/* IVirtualizationBackend implementation */

	virtual bool Initialize(const FString& ConfigEntry) override;

	virtual EConnectionStatus OnConnect() override;
	IVirtualizationBackend::EConnectionStatus OnConnectInternal(FString& InOutPort, FString& InOutUsername, bool bSaveConnectionSettings, FText& OutErrorMessage);
	
	virtual bool PushData(TArrayView<FPushRequest> Requests, EPushFlags Flags) override;
	virtual bool PullData(TArrayView<FPullRequest> Requests, EPullFlags Flags, FText& OutErrors) override;

	virtual bool DoesPayloadExist(const FIoHash& Id) override;
	
	virtual bool DoPayloadsExist(TArrayView<const FIoHash> PayloadIds, TArray<bool>& OutResults) override;

private:

	bool TryApplySettingsFromConfigFiles(const FString& ConfigEntry);

	void CreateDepotPath(const FIoHash& PayloadId, FStringBuilderBase& OutPath);

	bool FindSubmissionWorkingDir(const FString& ConfigEntry);

	/** Will display a FMessage notification to the user on the next valid engine tick to try and keep them aware of connection failures */
	void OnConnectionError(FText ErrorMessage);

	/** A source control connection owned by the backend*/
	TUniquePtr<ISourceControlProvider> SCCProvider;
	
	/** The name of the current project */
	FString ProjectName;

	/** The path (in depot format) to where the virtualized payloads are stored in source control */
	FString DepotPathRoot;

	/** Address of the server to connect to */
	FString ServerAddress;

	/** The stream containing the DepotRoot where the virtualized payloads are stored in source control */
	FString ClientStream;

	/** The root directory from which payloads are submitted. */
	FString SubmissionRootDir;

	/** The name to use for the p4 ignore file */
	FString IgnoreFileName = TEXT(".p4ignore.txt");

	/** Should we try to make the temp client partitioned or not? */
	bool bUsePartitionedClient = true;

	/** When true, the backend will not raise a pop up notification on connection error */
	bool bSuppressNotifications = false;

	/** The maximum number of files to send in a single source control operation */
	int32 MaxBatchCount = 100;

	/** A counted semaphore that will limit the number of concurrent connections that we can make */
	TUniquePtr<UE::Virtualization::FSemaphore> ConcurrentConnectionLimit;
	
	/** The number of times to retry pulling a payload from the depot */
	int32 RetryCount = 2;

	/** The length of time (in milliseconds) to wait after each pull attempt before retrying. */
	int32 RetryWaitTimeMS = 100;

	/** When true we allow the revision control provider to read settings from the users local SourceControlSettings.ini file */
	bool bUseLocalIniFileSettings = true;

	/* When true, if the initial connection to the revision control provider fails we will show a slate dialog (if possible) to the user prompting for correct settings */
	bool bUseRetryConnectionDialog = false;
};

} // namespace UE::Virtualization
