// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualizationSourceControlUtilities.h"

#include "Async/TaskGraphInterfaces.h"
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "ISourceControlRevision.h"
#include "Misc/PackagePath.h"
#include "Misc/PackageSegment.h"
#include "SourceControlOperations.h"

#define LOCTEXT_NAMESPACE "Virtualization"

// When enabled we push the source control work to the main thread via the task graph system.
// Note that when enabled this can potentially cause thread locks so is no a production ready 
// fix. Realistically we need to fix the SourceControl API to accept requests from any thread 
// which is currently under discussion.
#define UE_FORCE_SOURCECONTROL_TO_MAIN_THREAD 1

// This code relies on changes to the SourceControl API which need to be discussed, enabling this
// will allow the code to compile so it can be submitted without causing problems to others.
// The feature is entirely opt in so it shouldn't affect anyone else.
#define UE_FIX_COMPILE_ISSUES 1

namespace UE::Virtualization::Experimental
{

bool FVirtualizationSourceControlUtilities::SyncPayloadSidecarFile(const FPackagePath& PackagePath)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPayloadSideCarFunctionality::SyncPayloadSidecarFile);

	if (!ISourceControlModule::Get().IsEnabled())
	{
		UE_LOGF(LogVirtualization, Error, "Attempting to sync a .upayload for '%ls' but revision control is disabled!", *PackagePath.GetDebugName());
		return false;
	}

	// We only allow the perforce source control system as we have not adequately tested the others!
	const FName SourceControlName = ISourceControlModule::Get().GetProvider().GetName();
	if (SourceControlName != TEXT("Perforce"))
	{
		UE_LOGF(LogVirtualization, Error, "Attempting to sync a .upayload for '%ls' but revision control is '%ls' and only Perforce is currently supported!", *PackagePath.GetDebugName(), *SourceControlName.ToString());
		return false;
	}

#if UE_FORCE_SOURCECONTROL_TO_MAIN_THREAD
	// ISourceControlOperation commands must be invoked from the game thread so we need to push the work there
	// and then wait on the results.
	bool bResult = false;
	FGraphEventRef EventRef = FFunctionGraphTask::CreateAndDispatchWhenReady([this, &bResult, PackagePath]()
		{
			bResult = SyncPayloadSidecarFileInternal(PackagePath);
		}, TStatId(), nullptr, ENamedThreads::GameThread);

	FTaskGraphInterface::Get().WaitUntilTaskCompletes(EventRef);
	return bResult;
#else
	return SyncPayloadSidecarFileInternal(PackagePath);
#endif //UE_FORCE_SOURCECONTROL_TO_MAIN_THREAD
}

bool FVirtualizationSourceControlUtilities::SyncPayloadSidecarFileInternal(const FPackagePath& PackagePath)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPayloadSideCarFunctionality::SyncPayloadSidecarFileInternal);

	ISourceControlProvider& SCCProvider = ISourceControlModule::Get().GetProvider();

	const FString AssetFilePath = PackagePath.GetLocalFullPath(EPackageSegment::Header);
	const FString SidecarFilePath = PackagePath.GetLocalFullPath(EPackageSegment::PayloadSidecar);

	// Update the state of the .uasset file in the cache and store the history of the file
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPayloadSideCarFunctionality::SyncPayloadSidecarGameThread::UpdateStatus);

		TSharedRef<FUpdateStatus, ESPMode::ThreadSafe> UpdateStatusOperation = ISourceControlOperation::Create<FUpdateStatus>();
		UpdateStatusOperation->SetUpdateHistory(true);
		if (SCCProvider.Execute(UpdateStatusOperation, AssetFilePath) != ECommandResult::Succeeded)
		{
			UE_LOGF(LogVirtualization, Error, "Failed to update revision control state for '%ls'", *PackagePath.GetDebugName());
			return false;
		}
	}

	// Get the state of the .uasset from the cache
	FSourceControlStatePtr State;
	{ 
		TRACE_CPUPROFILER_EVENT_SCOPE(FPayloadSideCarFunctionality::SyncPayloadSidecarGameThread::GetState);
		State = SCCProvider.GetState(AssetFilePath, EStateCacheUsage::Use);
		if (!State)
		{
			UE_LOGF(LogVirtualization, Error, "Failed to find revision control state for '%ls'", *PackagePath.GetDebugName());
			return false;
		}
	}

	// Make sure we have access to a valid revision
#if UE_FIX_COMPILE_ISSUES
	checkf(false, TEXT("Attempting to call SyncPayloadSidecarFile before UE_FIX_COMPILE_ISSUES was fixed!"));
	const int32 LocalRevision = INDEX_NONE;
#else
	const int32 LocalRevision = State->GetLocalRevision();
#endif //UE_FIX_COMPILE_ISSUES

	if (LocalRevision == INDEX_NONE)
	{
		UE_LOGF(LogVirtualization, Error, "Failed to find revision control revision for '%ls'", *PackagePath.GetDebugName());
		return false;
	}

	// Now we can access the history for the revision that the .uasset is currently synced to
	TSharedPtr<ISourceControlRevision, ESPMode::ThreadSafe> Revision = State->FindHistoryRevision(LocalRevision);
	if (!Revision)
	{
		UE_LOGF(LogVirtualization, Error, "Failed to find the revision (%d) history for '%ls'", LocalRevision, *PackagePath.GetDebugName());
		return false;
	}

	// For P4 the CheckInIdentifier is actually the changelist that the revision was submitted under
	const int32 ChangelistNumber = Revision->GetCheckInIdentifier();
	if (ChangelistNumber == INDEX_NONE)
	{
		UE_LOGF(LogVirtualization, Error, "Revision (%d) history was invalid for '%ls'", LocalRevision, *PackagePath.GetDebugName());
		return false;
	}

	// The revision number passed to FSync should be the changelist number, not the file revision
	// NOTE: This will not force sync, if the file is the correct version in the workspace but has 
	// been deleted, this will not fix it!
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPayloadSideCarFunctionality::SyncPayloadSidecarGameThread::Sync);
	
		TSharedRef<FSync, ESPMode::ThreadSafe> SyncCommand = ISourceControlOperation::Create<FSync>();
		SyncCommand->SetRevision(FString::Printf(TEXT("%d"), ChangelistNumber));

		if (SCCProvider.Execute(SyncCommand, SidecarFilePath, EConcurrency::Asynchronous) == ECommandResult::Succeeded)
		{
			UE_LOGF(LogVirtualization, Verbose, "Successfully synced .upayload file for '%ls'", *PackagePath.GetDebugName());
			return true;
		}
		else
		{
			UE_LOGF(LogVirtualization, Error, "Failed to sync .upayload file for '%ls'", *PackagePath.GetDebugName());
			return false;
		}
	}
}

} // namespace UE::Virtualization::Experimental

namespace UE::Virtualization
{

bool TryCheckoutFiles(const TArray<FString>& FilesToCheckState, TArray<FText>& OutErrors, TArray<FString>* OutFilesCheckedOut)
{
	TArray<FSourceControlStateRef> PathStates;
	PathStates.Reserve(FilesToCheckState.Num());

	ISourceControlProvider& SCCProvider = ISourceControlModule::Get().GetProvider();

	// Early out if revision control is disabled
	if (!SCCProvider.IsEnabled())
	{
		return true;
	}

	// FCheckOut does not store info on exactly which files are checked out which means we cannot accurately fill in
	// OutFilesCheckedOut unless we first check on the file states.
	// TODO: Look into changing FCheckOut to provide the info we need to skip out this step.
	ECommandResult::Type UpdateResult = SCCProvider.GetState(FilesToCheckState, PathStates, EStateCacheUsage::ForceUpdate);
	if (UpdateResult != ECommandResult::Type::Succeeded)
	{
		FText Message = LOCTEXT("VA_FileState", "Failed to find the state of package files in revision control when trying to check them out");
		OutErrors.Add(MoveTemp(Message));

		return false;
	}

	TArray<FString> FilesToCheckout;
	FilesToCheckout.Reserve(PathStates.Num());

	for (const FSourceControlStateRef& State : PathStates)
	{
		if (State->IsSourceControlled() && !State->CanEdit() && State->CanCheckout())
		{
			FilesToCheckout.Add(State->GetFilename());
		}
	}

	if (!FilesToCheckout.IsEmpty())
	{
		ECommandResult::Type CheckoutResult = SCCProvider.Execute(ISourceControlOperation::Create<FCheckOut>(), FilesToCheckout);
		if (CheckoutResult == ECommandResult::Type::Succeeded)
		{
			if (OutFilesCheckedOut != nullptr)
			{
				*OutFilesCheckedOut = MoveTemp(FilesToCheckout);
			}	
		}
		else
		{
			FText Message = LOCTEXT("VA_Checkout", "Failed to checkout packages from revision control");
			OutErrors.Add(MoveTemp(Message));

			return false;
		}
	}

	return true;
}

} //namespace UE::Virtualization

#undef LOCTEXT_NAMESPACE
