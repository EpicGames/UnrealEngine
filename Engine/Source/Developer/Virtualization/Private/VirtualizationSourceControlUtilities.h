// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Virtualization/VirtualizationSystem.h"

namespace UE::Virtualization
{

/**
* Utility function to make it easier to check out a number of files from revision control. Note that
* the function does not do anything to files that are already checked out.
* 
 * @param FilesToCheckout		A list of files to check out of revision control
 * @param OutErrors				The function will place any errors encountered here
 * @param OutFilesCheckedOut	An optional array, which if provided will be filled in with the list
 *								of files that were actually checked out from revision control.
 * 
 * @return	True if the files were checked our OR revision control is not enabled. If revision control was enabled
  *			and we encountered errors, then the function will return false.
 */
bool TryCheckoutFiles(const TArray<FString>& FilesToCheckout, TArray<FText>& OutErrors, TArray<FString>* OutFilesCheckedOut);

} // UE::Virtualization

namespace UE::Virtualization::Experimental
{

class FVirtualizationSourceControlUtilities : public IVirtualizationSourceControlUtilities
{
public:
	FVirtualizationSourceControlUtilities() = default;
	~FVirtualizationSourceControlUtilities() = default;

private:
	/** 
	 * Wrapper call around SyncPayloadSidecarFileInternal which will either call it directly
	 * or attempt to marshal it to the main thread if needed 
	 */
	virtual bool SyncPayloadSidecarFile(const FPackagePath& PackagePath) override;

	/** Effectively the override of IVirtualizationSourceControlUtilities::SyncPayloadSidecarFile */
	bool SyncPayloadSidecarFileInternal(const FPackagePath& PackagePath);
};

} // namespace UE::Virtualization::Experimental
