// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualizationUtilities.h"
#include "VirtualizationExperimentalUtilities.h"

#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "IO/IoHash.h"
#include "Misc/App.h"
#include "Misc/Paths.h"
#include "Misc/StringBuilder.h"
#include "UObject/PackageFileSummary.h"
#include "UObject/PackageResourceManager.h"
#include "Virtualization/VirtualizationSystem.h"
#include "Virtualization/VirtualizationTypes.h"
#include "VirtualizationManager.h"

namespace UE::Virtualization::Utils
{

void PayloadIdToPath(const FIoHash& Id, FStringBuilderBase& OutPath)
{
	OutPath.Reset();
	OutPath << Id;

	TStringBuilder<10> Directory;
	Directory << OutPath.ToView().Left(2) << TEXT("/");
	Directory << OutPath.ToView().Mid(2, 2) << TEXT("/");
	Directory << OutPath.ToView().Mid(4, 2) << TEXT("/");

	OutPath.ReplaceAt(0, 6, Directory);

	OutPath << TEXT(".upayload");
}

FString PayloadIdToPath(const FIoHash& Id)
{
	TStringBuilder<52> Path;
	PayloadIdToPath(Id, Path);

	return FString(Path);
}

void GetFormattedSystemError(FStringBuilderBase& SystemErrorMessage)
{
	SystemErrorMessage.Reset();

	const uint32 SystemError = FPlatformMisc::GetLastError();
	// If we have a system error we can give a more informative error message but don't output it if the error is zero as 
	// this can lead to very confusing error messages.
	if (SystemError != 0)
	{
		TCHAR SystemErrorMsg[MAX_SPRINTF] = { 0 };
		FPlatformMisc::GetSystemErrorMessage(SystemErrorMsg, UE_ARRAY_COUNT(SystemErrorMsg), SystemError);

		SystemErrorMessage.Appendf(TEXT("'%s' (%d)"), SystemErrorMsg, SystemError);
	}
	else
	{
		SystemErrorMessage << TEXT("'unknown reason' (0)");
	}
}

ETrailerFailedReason FindTrailerFailedReason(const FPackagePath& PackagePath)
{
	TUniquePtr<FArchive> Ar = IPackageResourceManager::Get().OpenReadExternalResource(EPackageExternalResource::WorkspaceDomainFile, PackagePath.GetPackageName());

	if (!Ar)
	{
		return ETrailerFailedReason::NotFound;
	}

	FPackageFileSummary Summary;
	*Ar << Summary;

	if (Ar->IsError() || Summary.Tag != PACKAGE_FILE_TAG)
	{
		return ETrailerFailedReason::InvalidSummary;
	}

	if (Summary.GetFileVersionUE() < EUnrealEngineObjectUE5Version::PAYLOAD_TOC)
	{
		return ETrailerFailedReason::OutOfDate;
	}

	return ETrailerFailedReason::Unknown;
}

bool ExpandEnvironmentVariables(FStringView InputPath, FStringBuilderBase& OutExpandedPath)
{
	while (true)
	{
		const int32 EnvVarStart = InputPath.Find(TEXT("$("));
		if (EnvVarStart == INDEX_NONE)
		{
			// If we haven't expanded anything yet we can just copy the input path
			// If we have expanded then we need to append the remainder of the path
			if (OutExpandedPath.Len() == 0)
			{
				OutExpandedPath = InputPath;
				return true;
			}
			else
			{
				OutExpandedPath << InputPath;
				return true;
			}
		}

		const int32 EnvVarEnd = InputPath.Find(TEXT(")"), EnvVarStart + 2);
		const int32 EnvVarNameLength = EnvVarEnd - (EnvVarStart + 2);

		TStringBuilder<128> EnvVarName;
		EnvVarName = InputPath.Mid(EnvVarStart + 2, EnvVarNameLength);

		FString EnvVarValue;
		if (EnvVarName.ToView() == TEXT("Temp") || EnvVarName.ToView() == TEXT("Tmp"))
		{
			// On windows the temp envvar is often in 8.3 format
			// Either we need to expose ::GetLongPathName in some way or we need to consider
			// calling it in WindowsPlatformMisc::GetEnvironmentVariable.
			// Until we decide this is a quick work around, check for the Temp envvar and if
			// it is being requested us the ::UserTempDir function which will convert 8.3 
			// format correctly.
			// This should be solved before we consider moving this utility function into core
			EnvVarValue = FPlatformProcess::UserTempDir();

			FPaths::NormalizeDirectoryName(EnvVarValue);
		}
		else
		{
			EnvVarValue = FPlatformMisc::GetEnvironmentVariable(EnvVarName.ToString());
			if (EnvVarValue.IsEmpty())
			{
				UE_LOGF(LogVirtualization, Warning, "Could not find environment variable '%ls' to expand", EnvVarName.ToString());
				OutExpandedPath.Reset();
				return false;
			}
		}

		OutExpandedPath << InputPath.Mid(0, EnvVarStart);
		OutExpandedPath << EnvVarValue;

		InputPath = InputPath.Mid(EnvVarEnd + 1);
	}
}

bool IsProcessInteractive()
{
	if (FApp::IsUnattended())
	{
		return false;
	}

	if (IsRunningCommandlet())
	{
		return false;
	}

	// We used to check 'GIsRunningUnattendedScript' here as well but there
	// are a number of places in the editor enabling this global during which
	// the editor does stay interactive, such as when rendering thumbnail
	// images for the content browser.
	// Leaving this comment block here to show why we are not checking this
	// value anymore.

	if (IS_PROGRAM)
	{
		return false;
	}

	return true;
}

EPayloadFilterReason FixFilterFlags(FStringView PackagePath, uint64 SizeOnDisk, EPayloadFilterReason CurrentFilterFlags)
{
	if (IVirtualizationSystem::IsInitialized() && IVirtualizationSystem::GetSystemName() == FName("Default") && IVirtualizationSystem::Get().IsEnabled())
	{
		// Very hacky but should be safe if the system name is "Default". Allows us to do this without actually modifying the public API.
		FVirtualizationManager& Manager = static_cast<FVirtualizationManager&>(IVirtualizationSystem::Get());

		return Manager.FixFilterFlags(PackagePath, SizeOnDisk, CurrentFilterFlags);
	}

	return CurrentFilterFlags;
}

bool TryFindProject(const FString& PackagePath, const FString& ProjectExtension, FString& OutProjectFilePath, FString& OutPluginFilePath)
{
	return TryFindProject(PackagePath, MakeConstArrayView(&ProjectExtension, 1), OutProjectFilePath, OutPluginFilePath);
}

class FProjectFileVistitor : public IPlatformFile::FDirectoryVisitor
{
public:
	FProjectFileVistitor(TConstArrayView<FString> InProjectExtensions, TArray<FString>& OutProjectFiles)
		: ProjectExtensions(InProjectExtensions)
		, ProjectFiles(OutProjectFiles)
	{
	}

	virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
	{
		if (!bIsDirectory)
		{
			for (const FString& Extension : ProjectExtensions)
			{
				if (FStringView(FilenameOrDirectory).EndsWith(Extension))
				{
					ProjectFiles.Add(FilenameOrDirectory);
					return true;
				}
			}
		}

		return true;
	}

private:
	TConstArrayView<FString> ProjectExtensions;
	TArray<FString>& ProjectFiles;
};

bool TryFindProject(const FString& PackagePath, TConstArrayView<FString> ProjectExtensions, FString& ProjectFilePath, FString& PluginFilePath)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::Virtualization::Utils::TryFindProject);

	// TODO: This could be heavily optimized by caching known project files maybe with the use of FDirectoryTree
	// TODO: Relying on the known content & plugin directory conventions helps optimize this code but is fragile.
	// But if we started checking every directory then we'd need to start caching the results.
	int32 ContentIndex = PackagePath.Find(TEXT("/content/"), ESearchCase::IgnoreCase, ESearchDir::FromEnd);

	// Early out if there is not a single content directory in the path
	if (ContentIndex == INDEX_NONE)
	{
		UE_LOGF(LogVirtualization, Verbose, "'%ls' is not under a content directory", *PackagePath);
		return false;
	}

	while (ContentIndex != INDEX_NONE)
	{
		// Assume that the project directory is the parent of the /content/ directory
		FString ProjectDirectory = PackagePath.Left(ContentIndex);
		FString PluginDirectory;

		TArray<FString> ProjectFile;
		TArray<FString> PluginFile;

		FProjectFileVistitor ProjectFileVistitor(ProjectExtensions, ProjectFile);

		IFileManager::Get().IterateDirectory(*ProjectDirectory, ProjectFileVistitor);

		if (ProjectFile.IsEmpty())
		{
			// If there was no project file, the package could be in a plugin, so lets check for that
			PluginDirectory = ProjectDirectory;
			IFileManager::Get().FindFiles(PluginFile, *PluginDirectory, TEXT(".uplugin"));

			if (PluginFile.Num() == 1)
			{
				PluginFilePath = PluginDirectory / PluginFile[0];

				// We have a valid plugin file, so we should be able to find a /plugins/ directory which will be just below the project directory
				const int32 PluginIndex = PluginDirectory.Find(TEXT("/plugins/"), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
				if (PluginIndex != INDEX_NONE)
				{
					// We found the plugin root directory so the one above it should be the project directory
					ProjectDirectory = PluginDirectory.Left(PluginIndex);
					IFileManager::Get().IterateDirectory(*ProjectDirectory, ProjectFileVistitor);
				}
			}
			else if (PluginFile.Num() > 1)
			{
				UE_LOGF(LogVirtualization, Warning, "Found multiple .uplugin files for '%ls' at '%ls'", *PackagePath, *PluginDirectory);
				return false;
			}
		}

		if (ProjectFile.Num() == 1)
		{
			ProjectFilePath = ProjectFile[0];
			return true;
		}
		else if (!ProjectFile.IsEmpty())
		{
			UE_LOGF(LogVirtualization, Warning, "Found multiple .uproject files for '%ls' at '%ls'", *PackagePath, *ProjectDirectory);
			return false;
		}

		// Could be more than one content directory in the path so lets keep looking
		ContentIndex = PackagePath.Find(TEXT("/content/"), ESearchCase::IgnoreCase, ESearchDir::FromEnd, ContentIndex);
	}

	// We found one or more content directories but none of them contained a project file
	UE_LOGF(LogVirtualization, Verbose, "Failed to find project file for '%ls'", *PackagePath);

	return false;
}

} // namespace UE::Virtualization::Utils

