// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Mutex.h"
#include "Async/UniqueLock.h"
#include "Compression/CompressedBuffer.h"
#include "Containers/StringView.h"
#include "Features/IModularFeature.h"
#include "Features/IModularFeatures.h"
#include "Templates/UniquePtr.h"
#include "Virtualization/VirtualizationSystem.h"

struct FIoHash;

namespace UE::Virtualization
{
/**
 * The interface to derive from to create a new backend implementation.
 * 
 * Note that virtualization backends are instantiated FVirtualizationManager via  
 * IVirtualizationBackendFactory so each new backend derived from IVirtualizationBackend  
 * will also need a factory derived from IVirtualizationBackendFactory. You can either do
 * this manually or use the helper macro 'UE_REGISTER_VIRTUALIZATION_BACKEND_FACTORY' to 
 * generate the code for you.
 *
 */
class IVirtualizationBackend
{
public:
	/** Enum detailing which operations a backend can support */
	enum class EOperations : uint8
	{
		/** Supports no operations, this should only occur when debug settings are applied */
		None = 0,
		/** Supports only push operations */
		Push = 1 << 0,
		/** Supports only pull operations */
		Pull = 1 << 1,
	};

	FRIEND_ENUM_CLASS_FLAGS(EOperations);

	/** Status of the backends connection to its services */
	enum class EConnectionStatus : uint8
	{
		/** No connection attempt has been made */
		None,
		/** The previous connection attempt ended with an error */
		Error,
		/** The connection has been made successfully */
		Connected,
	};

	/** Specialize pushing behavior, @See IVirtualizationBackend::PushData */
	enum class EPushFlags : uint32
	{
		None = 0,
		/** Backends should not test payloads for existence and just assume that they need re-uploading */
		Force = 1 << 0
	};

	FRIEND_ENUM_CLASS_FLAGS(EPushFlags);

	/** Specialize pulling behavior, @See IVirtualizationBackend::PullData */
	enum class EPullFlags : uint32
	{
		None = 0
	};

	FRIEND_ENUM_CLASS_FLAGS(EPullFlags);

protected:

	IVirtualizationBackend(FStringView InConfigName, FStringView InDebugName, EOperations InSupportedOperations)
		: ConfigName(InConfigName)
		, DebugName(InDebugName)
		, SupportedOperations(InSupportedOperations)
		, DebugDisabledOperations(EOperations::None)
	{
		checkf(InSupportedOperations != EOperations::None, TEXT("Cannot create a backend with no supported operations!"));
	}

public:
	virtual ~IVirtualizationBackend() = default;

	/**
	 * This will be called during the setup of the backend hierarchy. The entry config file
	 * entry that caused the backend to be created will be passed to the method so that any
	 * additional settings may be parsed from it.
	 * Take care to clearly log any error that occurs so that the end user has a clear way
	 * to fix them.
	 *
	 * @param ConfigEntry	The entry for the backend from the config ini file that may
	 *						contain additional settings.
	 * @return				Returning false indicates that initialization failed in a way
	 *						that the backend will not be able to function correctly.
	 */
	virtual bool Initialize(const FString& ConfigEntry) = 0;

	
	UE_DEPRECATED(5.6, "Call IVirtualizationBackend::TryConnect instead")
	virtual void Connect() final
	{
		TryConnect(false);
	}

	/** 
	 * Attempt to connect the backend to its services if not already connected.
	 * 
	 * @praram bAllowReconnect	When true the backend will attempt to connect to it's services even
	 *							if it has previously failed to do so. When false the backend will
	 *							only ever attempt to connect a single time with that result being
	 *							final.
	 * 
	 * @return The current connection status of the backend. @see EConnectionStatus
	 */
	EConnectionStatus TryConnect(bool bAllowReconnect)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(IVirtualizationBackend::TryConnect);

		UE::TUniqueLock _(ConnectionMutex);

		if ((bAllowReconnect && ConnectionStatus != EConnectionStatus::Connected) || (!bAllowReconnect && ConnectionStatus == EConnectionStatus::None))
		{
			ConnectionStatus = OnConnect();
		}
		
		return ConnectionStatus;
	}

	/**
	 * Returns the connection status of the backend
	 * 
	 * @return @see EConnectionStatus
	 */
	EConnectionStatus GetConnectionStatus() const
	{
		UE::TUniqueLock _(ConnectionMutex);
		return ConnectionStatus;
	}

	/**
	 * @return True if all of the requests succeeded, false if one of more failed
	 */
	virtual bool PushData(TArrayView<FPushRequest> Requests, EPushFlags Flags) = 0;

	/** 
	 * The backend will attempt to retrieve the given payloads by what ever method the backend uses.
	 * 
	 * It should be assumed that the list of requests will not contain any duplicate or invalid
	 * payload identifiers so there is no need for each backend to perform validation.
	 * 
	 * It should be assumed that the caller will validate all payloads that are successfully pulled
	 * to make sure that match the requested payload identifiers  so there is no need for each 
	 * backend to do this.
	 * 
	 * @param	Requests	An array of payload pull requests. @see FPullRequest
	 * @param	Flags		Flags that allow the pull to be specialized @see EPullFlags
	 * @param	OutErrors	If the pull fails, the backend can add optional additional error messages
	 *						to be displayed directly to the user.
	 * 
	 * @return				True if no errors were encountered while pulling, otherwise false.
	 *						Note that returning true does not mean that all of the payloads were
	 *						found as a missing payload should not be considered an error condition.
	 */

	// Assume that the array only has unique requests
	// Will set request error value if there is a problem with loading it
	// Returns false on critical error, otherwise true
	virtual bool PullData(TArrayView<FPullRequest> Requests, EPullFlags Flags, FText& OutErrors) = 0;
	
	/**
	 * Checks if a payload exists in the backends storage.
	 * 
	 * @param	Id	The identifier of the payload to check
	 * 
	 * @return True if the backend storage already contains the payload, otherwise false
	 */
	virtual bool DoesPayloadExist(const FIoHash& Id) = 0;
	
	/**
	 * Checks if a number of payload exists in the backends storage.
	 *
	 * @param[in]	PayloadIds	An array of FIoHash that should be checked
	 * @param[out]	OutResults	An array to contain the result, true if the payload
	 *							exists in the backends storage, false if not.
	 *							This array will be resized to match the size of PayloadIds.
	 * 
	 * @return True if the operation completed without error, otherwise false
	 */
	virtual bool DoPayloadsExist(TArrayView<const FIoHash> PayloadIds, TArray<bool>& OutResults)
	{
		// This is the default implementation that just calls ::DoesExist on each FIoHash in the
		// array, one at a time. 
		// Backends may override this with their own implementations if it can be done with less
		// overhead by performing the check on the entire batch instead.

		OutResults.SetNum(PayloadIds.Num());

		for (int32 Index = 0; Index < PayloadIds.Num(); ++Index)
		{
			OutResults[Index] = DoesPayloadExist(PayloadIds[Index]);
		}

		return true;
	}

	/**
	 * Disables operations on the backend
	 * @param Operation	The operation(s) to be disabled
	 * 
	 * @return	True if at least one previously enabled operation was disabled by the call
	 *			False if there was no change.
	 */
	bool DisableOperation(EOperations Operation)
	{
		EOperations OriginalValue = SupportedOperations;

		EnumRemoveFlags(SupportedOperations, Operation);

		return OriginalValue != SupportedOperations;
	}
	
	/** 
	 * Returns true if the given operation is supported, this is set when the backend is created
	 * and should not change over it's life time.
	 */
	bool IsOperationSupported(EOperations Operation) const
	{
		return EnumHasAnyFlags(SupportedOperations, Operation);
	}

	/** Enable or disable the given operation based on the 'bIsDisabled' parameter */
	void SetOperationDebugState(EOperations Operation, bool bIsDisabled)
	{
		if (bIsDisabled)
		{
			EnumAddFlags(DebugDisabledOperations, Operation);
			
		}
		else
		{
			EnumRemoveFlags(DebugDisabledOperations, Operation);
		}
	}

	/** Returns true if the given operation is disabled for debugging purposes */
	bool IsOperationDebugDisabled(EOperations Operation) const
	{
		return EnumHasAnyFlags(DebugDisabledOperations, Operation);
	}

	/** Returns a string containing the name of the backend as it appears in the virtualization graph in the config file */
	const FString& GetConfigName() const
	{
		return ConfigName;
	}

	/** Returns a string that can be used to identify the backend for debugging and logging purposes */
	const FString& GetDebugName() const
	{
		return DebugName;
	}

private:

	/** Override to implement the backends connection code */
	virtual EConnectionStatus OnConnect() = 0;

private:

	/** The name assigned to the backend by the virtualization graph */
	FString ConfigName;

	/** Combination of the backend type and the name used to create it in the virtualization graph */
	FString DebugName;

	/** Protects threaded access to 'ConnectionStatus' */
	mutable UE::FMutex ConnectionMutex;

	/** The status of the connection to the backends service */
	EConnectionStatus ConnectionStatus = EConnectionStatus::None;

	/** The operations that this backend supports */
	EOperations SupportedOperations;

	/** The operations that have been disabled for debugging purposes */
	EOperations DebugDisabledOperations;
};

/** 
 * Derive from this interface to implement a factory to return a backend type.
 * An instance of the factory should be created and then registered to 
 * IModularFeatures with the feature name "VirtualizationBackendFactory" to
 * give 'FVirtualizationManager' access to it. 
 * The macro 'UE_REGISTER_VIRTUALIZATION_BACKEND_FACTORY' can be used to create 
 * a factory easily if you do not want to specialize the behavior.
*/
class IVirtualizationBackendFactory : public IModularFeature
{
public:
	/** 
	 * Creates a new backend instance.
	 * 
	 * @param ProjectName	The name of the current project
	 * @param ConfigName	The name given to the back end in the config ini file
	 * @return A new backend instance
	 */
	virtual TUniquePtr<IVirtualizationBackend> CreateInstance(FStringView ProjectName, FStringView ConfigName) = 0;

	/** Returns the name used to identify the type in config ini files */
	virtual FName GetName() = 0;
};

ENUM_CLASS_FLAGS(IVirtualizationBackend::EOperations);

/**
 * This macro is used to generate a backend factories boilerplate code if you do not
 * need anything more than the default behavior.
 * As well as creating the class, a single instance will be created which will register the factory with
 * 'IModularFeatures' so that it is ready for use.
 * 
 * @param BackendClass The name of the class derived from 'IVirtualizationBackend' that the factory should create
 * @param The name used in config ini files to reference this backend type.
 */
#define UE_REGISTER_VIRTUALIZATION_BACKEND_FACTORY(BackendClass, ConfigName) \
	class F##BackendClass##Factory : public UE::Virtualization::IVirtualizationBackendFactory \
	{ \
	public: \
		F##BackendClass##Factory() { IModularFeatures::Get().RegisterModularFeature(FName("VirtualizationBackendFactory"), this); }\
		virtual ~F##BackendClass##Factory() { IModularFeatures::Get().UnregisterModularFeature(FName("VirtualizationBackendFactory"), this); } \
	private: \
		virtual TUniquePtr<UE::Virtualization::IVirtualizationBackend> CreateInstance(FStringView ProjectName, FStringView ConfigName) override \
		{ \
			return MakeUnique<BackendClass>(ProjectName, ConfigName, WriteToString<256>(#ConfigName, TEXT(" - "), ConfigName).ToString()); \
		} \
		virtual FName GetName() override { return FName(#ConfigName); } \
	}; \
	static F##BackendClass##Factory BackendClass##Factory##Instance;

#define UE_REGISTER_VIRTUALIZATION_BACKEND_FACTORY_LEGACY_IMPL(FactoryName, BackendClass, LegacyConfigName, ConfigName) \
	class F##FactoryName : public UE::Virtualization::IVirtualizationBackendFactory \
	{ \
	public: \
		F##FactoryName(const TCHAR* InLegacyConfigName, const TCHAR* InNewConfigName) \
			: StoredLegacyConfigName(InLegacyConfigName) \
			, StoredNewConfigName(InNewConfigName) \
			{ IModularFeatures::Get().RegisterModularFeature(FName("VirtualizationBackendFactory"), this); }\
		virtual ~F##FactoryName() { IModularFeatures::Get().UnregisterModularFeature(FName("VirtualizationBackendFactory"), this); } \
	private: \
		virtual TUniquePtr<UE::Virtualization::IVirtualizationBackend> CreateInstance(FStringView ProjectName, FStringView ConfigName) override \
		{ \
			UE_LOGF(LogVirtualization, Warning, "Creating a backend via the legacy config name '%ls' use '%ls' instead", *StoredLegacyConfigName, *StoredNewConfigName); \
			return MakeUnique<BackendClass>(ProjectName, ConfigName, WriteToString<256>(#ConfigName, TEXT(" - "), ConfigName).ToString()); \
		} \
		virtual FName GetName() override { return FName(#LegacyConfigName); } \
		FString StoredLegacyConfigName; \
		FString StoredNewConfigName;\
	}; \
	static F##FactoryName FactoryName##Instance(TEXT(#LegacyConfigName), TEXT(#ConfigName));

/** 
 * This macro can be used to change the config name used to create backends while allowing older config file entries to continue working.
 * If this factory is used to instantiate a backend then we will log a warning to the user so that they can update their config file.
 * 
 * @param BackendClass		The name of the class derived from 'IVirtualizationBackend' that the factory should create.
 * @param LegacyConfigName	The old name that 'ConfigName' is now replacing.
 * @param ConfigName		The name used in config ini files to reference this backend type.
 */
#define UE_REGISTER_VIRTUALIZATION_BACKEND_FACTORY_LEGACY(BackendClass, LegacyConfigName, ConfigName) \
	UE_REGISTER_VIRTUALIZATION_BACKEND_FACTORY_LEGACY_IMPL(BackendClass##LegacyConfigName##To##ConfigName##Factory, BackendClass, LegacyConfigName, ConfigName)

} // namespace UE::Virtualization
