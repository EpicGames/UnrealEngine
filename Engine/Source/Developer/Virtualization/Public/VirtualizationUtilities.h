// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"

class FPackagePath;

struct FIoHash;

#define UE_API VIRTUALIZATION_API

namespace UE::Virtualization::Utils
{

/** 
 * Converts a FIoHash into a file path.
 *
 * This utility will take an FIoHash and return a file path that is
 * 3 directories deep. The first six characters of the id will be used to
 * create the directory names, with each directory using two characters.
 * The remaining thirty four characters will be used as the file name.
 * Lastly the extension '.payload' will be applied to complete the path/
 * Example: FIoHash 0139d6d5d477e32dfd2abd3c5bc8ea8507e8eef8 becomes
 *			01/39/d6/d5d477e32dfd2abd3c5bc8ea8507e8eef8.payload'
 * 
 * @param	Id The payload identifier used to create the file path.
 * @param	OutPath Will be reset and then assigned the resulting file path.
 *			The string builder must have a capacity of at least 52 characters 
			to avoid reallocation.
 */
UE_API void PayloadIdToPath(const FIoHash& Id, FStringBuilderBase& OutPath);

/** 
 * Converts a FIoHash into a file path.
 * 
 * See above for further details
 * 
 * @param	Id The payload identifier used to create the file path.
 * @return	The resulting file path.
 */
UE_API FString PayloadIdToPath(const FIoHash& Id);

/**
 * Fill in the given string builder with the human readable message of the current system
 * code, followed by the code value itself.
 * In the system value is currently 0, then we assume that it was cleared before this was
 * able to be called and write that the error is unknown instead of assuming that the
 * operation was a success.
 */
UE_API void GetFormattedSystemError(FStringBuilderBase& SystemErrorMessage);


enum class ETrailerFailedReason : uint8
{
	/** Could not open the package for reading */
	NotFound,
	/** The header of the package (summary) could not be read or was otherwise corrupted */
	InvalidSummary,
	/** The package predates package version 1002 in which the package trailer was introduced */
	OutOfDate,
	/** The reason for the missing package trailer could not be determined */
	Unknown
};

/** 
 * Utility that returns the reason why a given package does not have a package trailer.
 * Note that it does not actually try to load the trailer or validate that the trailer
 * is in fact missing and assumes that the caller previously attempted to a call to
 * load a trailer from the package but it failed.
 */
UE_API ETrailerFailedReason FindTrailerFailedReason(const FPackagePath& PackagePath);

/** 
 * Parse the given path and expand any environment variables found with the format
 * $(EnvVarName). If the variable cannot be found the function will return false 
 * and OutExpandedPath will be reset.
 */
UE_API bool ExpandEnvironmentVariables(FStringView InputPath, FStringBuilderBase& OutExpandedPath);

/** 
 * Returns true if the process is interactive and we should display error dialogs to
 * the user.
 * */
UE_API bool IsProcessInteractive();

/**
 * Find the project file for the project that contains the provided package file. Note that this only intended to work for package files as it makes
 * several assumptions about the path conventions in order to speed up the search.
 * 
 * @PackagePath			Path of a package file to start searching from. The file must be stored under a /content/ directory to be considered valid.
 * @ProjectExtension	The extension that the project file may have.
 * @OutProjectFilePath	The project path will be written to this string if found.
 * @OutPluginFilePath	If the package is stored in a plugin then the path to it's '.uplugin' file will be written to this string.
 * 
 * @return				True if a project file is found otherwise it will return false.
 */
UE_API bool TryFindProject(const FString& PackagePath, const FString& ProjectExtension, FString& OutProjectFilePath, FString& OutPluginFilePath);

/**
 * Find the project file for the project that contains the provided package file. Note that this only intended to work for package files as it makes
 * several assumptions about the path conventions in order to speed up the search.
 *
 * @PackagePath			Path of a package file to start searching from. The file must be stored under a /content/ directory to be considered valid.
 * @ProjectExtensions	An array of potential extensions that the project file may have.
 * @OutProjectFilePath	The project path will be written to this string if found.
 * @OutPluginFilePath	If the package is stored in a plugin then the path to it's '.uplugin' file will be written to this string.
 *
 * @return				True if a project file is found otherwise it will return false.
 */
UE_API bool TryFindProject(const FString& PackagePath, TConstArrayView<FString> ProjectExtensions, FString& OutProjectFilePath, FString& OutPluginFilePath);

} // namespace UE::Virtualization::Utils

#undef UE_API
