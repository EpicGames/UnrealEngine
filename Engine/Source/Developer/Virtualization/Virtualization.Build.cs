// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class Virtualization : ModuleRules
{
	public Virtualization(ReadOnlyTargetRules Target) : base(Target)
	{
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

		PrivateIncludePathModuleNames.Add("Analytics");

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"DerivedDataCache",
				"Projects",
				"SourceControl"
			}
		);

		if (Target.bUsesSlate)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"MessageLog",
					"Slate",
					"SlateCore"
				}
			);

			PublicDefinitions.Add("UE_VA_WITH_SLATE=1");
		}
		else
		{
			PublicDefinitions.Add("UE_VA_WITH_SLATE=0");
		}
	}
}
